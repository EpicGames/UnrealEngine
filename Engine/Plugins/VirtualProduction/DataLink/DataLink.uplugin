{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Motion Design Data Link",
	"Description": "",
	"Category": "Other",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "DataLink",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataLinkEdGraph",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataLinkEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataLinkHttp",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataLinkJson",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataLinkJsonEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataLinkDataTable",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataLinkWebSocket",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}