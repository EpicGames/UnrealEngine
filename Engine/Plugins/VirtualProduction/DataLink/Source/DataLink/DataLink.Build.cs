// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataLink : ModuleRules
{
	public DataLink(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
			}
		);

		PublicDefinitions.Add("WITH_DATALINK_CONTEXT=!NO_LOGGING");
	}
}
