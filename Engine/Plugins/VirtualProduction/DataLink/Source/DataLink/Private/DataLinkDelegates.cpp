// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkDelegates.h"

namespace UE::DataLink
{
	FOnRequestGraphCompilation OnRequestCompilation;

} // UE::DataLink
