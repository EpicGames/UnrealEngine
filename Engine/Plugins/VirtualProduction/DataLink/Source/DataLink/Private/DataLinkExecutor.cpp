// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkExecutor.h"
#include "DataLinkEnums.h"
#include "DataLinkExecutorArguments.h"
#include "DataLinkGraph.h"
#include "DataLinkLog.h"
#include "DataLinkNode.h"
#include "DataLinkPinReference.h"
#include "DataLinkProcessor.h"
#include "DataLinkSink.h"
#include "HAL/IConsoleManager.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"

namespace UE::DataLink::Private
{
	static TAutoConsoleVariable<bool> CVarDataLinkDebugNodeData(TEXT("DataLink.PrintNodeData")
		, /*DefaultValue*/false
		, TEXT("Prints the input and output data structs for each node executed."));
}

TSharedPtr<FDataLinkExecutor> FDataLinkExecutor::Create(FDataLinkExecutorArguments&& InArgs)
{
	TSharedRef<FDataLinkExecutor> Executor = MakeShared<FDataLinkExecutor>(FPrivateToken());

#if WITH_DATALINK_CONTEXT
	Executor->ContextName = MoveTemp(InArgs.ContextName);
#endif
	Executor->ContextObject = MoveTemp(InArgs.ContextObject);
	Executor->Instance = MoveTemp(InArgs.Instance);
	Executor->Sink = MoveTemp(InArgs.Sink);
	Executor->OutputProcessors = MoveTemp(InArgs.OutputProcessors);
	Executor->OnOutputData = MoveTemp(InArgs.OnOutputData);
	Executor->OnExecutionFinished = MoveTemp(InArgs.OnExecutionFinished);

	// Remove invalid entries
	Executor->OutputProcessors.RemoveAll(
		[](UDataLinkProcessor* InProcessor)
		{
			return !InProcessor;
		});

	return Executor;
}

FDataLinkExecutor::FDataLinkExecutor(FPrivateToken)
	: ExecutorId(FGuid::NewGuid())
{
}

FDataLinkExecutor::~FDataLinkExecutor()
{
	if (bRunning && UObjectInitialized())
	{
		bRunning = false;
		StopInternal();
	}
}

const FGuid& FDataLinkExecutor::GetExecutorId() const
{
	return ExecutorId;
}

FStringView FDataLinkExecutor::GetContextName() const
{
#if WITH_DATALINK_CONTEXT
	return ContextName;
#else
	return FStringView();
#endif
}

UObject* FDataLinkExecutor::GetContextObject() const
{
	return ContextObject;
}

FString FDataLinkExecutor::GetReferencerName() const
{
	return TEXT("FDataLinkExecutor");
}

void FDataLinkExecutor::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(ContextObject);
	InCollector.AddPropertyReferencesWithStructARO(FDataLinkInstance::StaticStruct(), &Instance);

	if (Sink.IsValid())
	{
		Sink->AddStructReferencedObjects(InCollector);
	}

	for (TPair<TObjectPtr<const UDataLinkNode>, FDataLinkNodeInstance>& Pair : NodeInstanceMap)
	{
		Pair.Value.AddReferencedObjects(InCollector);
	}

	InCollector.AddReferencedObjects(OutputProcessors);
}

void FDataLinkExecutor::Run()
{
	if (bRunning)
	{
		return;
	}
	bRunning = true;

	if (!ValidateRun())
	{
		BroadcastFailure();
		return;
	}

	// Ensure there's a valid sink instance
	if (!Sink.IsValid())
	{
		Sink = MakeShared<FDataLinkSink>();
	}

	// Hold onto an extra shared reference in case the execution finishes immediately
	TSharedRef<FDataLinkExecutor> This = SharedThis(this);

	for (UDataLinkProcessor* OutputProcessor : OutputProcessors)
	{
		OutputProcessor->Initialize(*this);
	}

	if (!ExecuteEntryNodes())
	{
		BroadcastFailure();
	}
}

void FDataLinkExecutor::Stop(EDataLinkExecutionResult InExecutionResult)
{
	if (!bRunning)
	{
		return;
	}
	bRunning = false;

	StopInternal();

	// Hold onto an extra shared reference to allow implementers to reset their held shared reference in the finish delegate
	// without destroying this
	const TSharedRef<FDataLinkExecutor> This = SharedThis(this);

	for (UDataLinkProcessor* OutputProcessor : OutputProcessors)
	{
		OutputProcessor->Finalize(*this, InExecutionResult);
	}

	OnExecutionFinished.ExecuteIfBound(*this, InExecutionResult);
}

const FDataLinkNodeInstance& FDataLinkExecutor::GetNodeInstance(const UDataLinkNode* InNode) const
{
	return NodeInstanceMap.FindChecked(InNode);
}

FDataLinkNodeInstance& FDataLinkExecutor::GetNodeInstanceMutable(const UDataLinkNode* InNode)
{
	return NodeInstanceMap.FindChecked(InNode);
}

FDataLinkNodeInstance* FDataLinkExecutor::FindNodeInstanceMutable(const UDataLinkNode* InNode)
{
	return NodeInstanceMap.Find(InNode);
}

void FDataLinkExecutor::Next(const UDataLinkNode* InNode)
{
	if (!ValidateNodeExecutionStatus(InNode))
	{
		BroadcastFailure();
		return;
	}

	TConstArrayView<FDataLinkPin> OutputPins = InNode->GetOutputPins();

	TArray<FDataLinkPinReference, TInlineAllocator<1>> InputPinsToExecute;
	InputPinsToExecute.Reserve(OutputPins.Num());

	TArray<FConstStructView, TInlineAllocator<1>> InputDataViews;
	InputDataViews.Reserve(OutputPins.Num());

	const FDataLinkOutputDataViewer& OutputDataViewer = GetNodeInstance(InNode).GetOutputDataViewer();

	for (const FDataLinkPin& OutputPin : OutputPins)
	{
		if (const FDataLinkPin* InputPin = OutputPin.GetLinkedInputPin())
		{
			// If the Linked Input Pin was gotten successfully, then it means that the Output Pin's Linked Node was valid
			check(OutputPin.LinkedNode);
			InputPinsToExecute.Emplace(OutputPin.LinkedNode, InputPin);
			InputDataViews.Emplace(OutputDataViewer.Find(OutputPin.Name));
		}
	}

	// No next node, broadcast output data
	if (InputPinsToExecute.IsEmpty())
	{
		// If the next node is null, it should be the output node
		if (!ensureAlways(Instance.DataLinkGraph && Instance.DataLinkGraph->GetOutputNode() == InNode))
		{
			UE_LOGF(LogDataLink, Error, "[%ls] Node ('%ls') has no node to go to and it's not an output node!", GetContextName().GetData(), *InNode->GetName());
			BroadcastFailure();
			return;
		}

		FStructView ResultDataView = OutputDataViewer.Find(Instance.DataLinkGraph->GetOutputPinName());
		BroadcastOutputData(ResultDataView);
		return;
	}

	if (!ExecuteInputPins(InputPinsToExecute, InputDataViews))
	{
		BroadcastFailure();
	}
}

void FDataLinkExecutor::NextPersist(const UDataLinkNode* InNode)
{
	bPersistExecution = true;
	Next(InNode);
}

void FDataLinkExecutor::Fail(const UDataLinkNode* InNode)
{
	ValidateNodeExecutionStatus(InNode);

	UE_LOGF(LogDataLink, Log, "[%ls] Node ('%ls') has failed to execute. Data Link execution finished.", GetContextName().GetData(), *InNode->GetName());
	BroadcastFailure();
}

bool FDataLinkExecutor::ValidateRun()
{
	if (!Instance.DataLinkGraph)
	{
		UE_LOGF(LogDataLink, Error, "[%.*ls] Invalid Data Link specified!"
			, GetContextName().Len()
			, GetContextName().GetData());
		return false;
	}

	if (!ValidateGraphOutputNode())
	{
		return false;
	}

	const int32 InputPinCount = Instance.DataLinkGraph->GetInputPinCount();

	if (Instance.InputData.Num() != InputPinCount)
	{
		UE_LOGF(LogDataLink, Error, "[%.*ls] DataLink Graph '%ls' requires %d inputs but %d were provided"
			, GetContextName().Len()
			, GetContextName().GetData()
			, *Instance.DataLinkGraph->GetName()
			, InputPinCount
			, Instance.InputData.Num());
		return false;
	}

	int32 PinIndex = 0;

	// Ensure the input pins are compatible with the input data
	const bool bResult = Instance.DataLinkGraph->ForEachInputPin(
		[&PinIndex, this](FDataLinkPinReference InPinReference)->bool
		{
			const int32 CurrentPinIndex = PinIndex++;

			if (InPinReference.Pin->Struct && InPinReference.Pin->Struct != Instance.InputData[CurrentPinIndex].Data.GetScriptStruct())
			{
				UE_LOGF(LogDataLink, Error, "[%.*ls] Input pin (%ls': input struct '%ls', owner '%ls') is not compatible with input data view '%ls'"
					, GetContextName().Len()
					, GetContextName().GetData()
					, *InPinReference.Pin->Name.ToString()
					, *InPinReference.Pin->Struct->GetName()
					, *InPinReference.OwningNode->GetName()
					, *GetNameSafe(Instance.InputData[CurrentPinIndex].Data.GetScriptStruct()));
				return false;
			}

			return true;
		});

	return bResult;
}

bool FDataLinkExecutor::ValidateGraphOutputNode()
{
	using namespace UE::DataLink;

	// Already verified to be valid
	check(Instance.DataLinkGraph);

	const UDataLinkNode* OutputNode = Instance.DataLinkGraph->GetOutputNode();
	if (!OutputNode)
	{
		UE_LOGF(LogDataLink, Error, "[%.*ls] Invalid Data Link output node! Graph: %ls"
			, GetContextName().Len()
			, GetContextName().GetData()
			, *Instance.DataLinkGraph->GetName());
		return false;
	}

	// If there's no valid output pin with the stored name then try to compile graph.
	// This should only happen for old graphs before output pin name was introduced.
	const FName OutputPinName = Instance.DataLinkGraph->GetOutputPinName();
	if (!OutputNode->GetOutputPins().Contains(OutputPinName))
	{
		// Todo: consider possible option in executor arguments to not do any compilation even if the graph is in a bad state and just fail execution 
		if (UE::DataLink::OnRequestCompilation.IsBound())
		{
			UE_LOGF(LogDataLink, Warning, "[%.*ls] Output pin name '%ls' was not found as a valid pin name in output node '%ls' in Data Link Graph '%ls'. Recompiling graph..."
				, GetContextName().Len()
				, GetContextName().GetData()
				, *OutputPinName.ToString()
				, *OutputNode->GetName()
				, *Instance.DataLinkGraph->GetName());

			const EGraphCompileStatus Result = UE::DataLink::OnRequestCompilation.Execute(Instance.DataLinkGraph);
			if (Result != EGraphCompileStatus::UpToDate)
			{
				UE_LOGF(LogDataLink, Error, "[%.*ls] Graph '%ls' failed to compile. Recompile graph manually."
					, GetContextName().Len()
					, GetContextName().GetData()
					, *Instance.DataLinkGraph->GetName());
				return false;
			}
		}
		else
		{
			UE_LOGF(LogDataLink, Error, "[%.*ls] Output pin name '%ls' was not found as a valid pin name in output node '%ls' in Data Link Graph '%ls'. Recompile Graph manually."
				, GetContextName().Len()
				, GetContextName().GetData()
				, *OutputPinName.ToString()
				, *OutputNode->GetName()
				, *Instance.DataLinkGraph->GetName());
			return false;
		}
	}

	return true;
}

bool FDataLinkExecutor::ValidateInputPins(TConstArrayView<FDataLinkPinReference> InInputPins, TConstArrayView<FConstStructView> InInputDataViews) const
{
	if (!ensureAlways(InInputPins.Num() == InInputDataViews.Num()))
	{
		UE_LOGF(LogDataLink, Error, "[%ls] Input Pin count %d does not match Input Data View %d"
			, GetContextName().GetData()
			, InInputPins.Num()
			, InInputDataViews.Num());
		return false;
	}

	for (int32 InputIndex = 0; InputIndex < InInputPins.Num(); ++InputIndex)
	{
		const FDataLinkPin* InputPin = InInputPins[InputIndex].Pin;
		check(InputPin);

		const FConstStructView& InputData = InInputDataViews[InputIndex];
		if (InputPin->Struct && InputPin->Struct != InputData.GetScriptStruct())
		{
			UE_LOGF(LogDataLink, Error, "[%ls] Input Data struct '%ls' does not match expected input pin '%ls' of type '%ls'"
				, GetContextName().GetData()
				, *GetNameSafe(InputData.GetScriptStruct())
				, *InputPin->Name.ToString()
				, *InputPin->Struct->GetName());
			return false;
		}
	}

	return true;
}

bool FDataLinkExecutor::ExecuteEntryNodes()
{
	const TArray<FDataLinkPinReference> InputPinsToExecute = Instance.DataLinkGraph->GetInputPins();

	uint16 NodesExecuted = 0;
	if (!ExecuteInputPins(InputPinsToExecute, Instance.GetInputDataViews(), &NodesExecuted))
	{
		return false;
	}

	// Execute Entry nodes with no input pins (i.e. without dependency) as these were not included in the initial input pins to execute
	for (const UDataLinkNode* InputNode : Instance.DataLinkGraph->GetInputNodes())
	{
		if (InputNode->GetInputPins().IsEmpty())
		{
			// Ensure there's a Node Instance to this Input Node
			FindOrAddNodeInstance(InputNode);

			if (!ExecuteNode(*InputNode))
			{
				return false;
			}

			++NodesExecuted;
		}
	}

	return NodesExecuted > 0;
}

bool FDataLinkExecutor::ExecuteInputPins(TConstArrayView<FDataLinkPinReference> InInputPins, TConstArrayView<FConstStructView> InInputDataViews, uint16* OutNodesExecuted)
{
	if (!ValidateInputPins(InInputPins, InInputDataViews))
	{
		return false;
	}

	// Struct bundling the pin and the data that will be fed into the pin
	struct FPinData
	{
		const FDataLinkPin* Pin;
		FConstStructView DataView;
	};

	// Struct bundling the node and the pin data that will be fed into the node
	struct FNodeData
	{
		bool operator==(const UDataLinkNode* InNode) const
		{
			return InNode == Node;
		}
		const UDataLinkNode* Node = nullptr;
		TArray<FPinData> PinData;
	};

	TArray<FNodeData> NodeData;

	// Group the pins per node
	for (int32 Index = 0; Index < InInputPins.Num(); ++Index)
	{
		const FDataLinkPinReference& InputPinReference = InInputPins[Index];
		check(InputPinReference.Pin);

		FNodeData* NodeDataEntry = NodeData.FindByKey(InputPinReference.OwningNode);
		if (!NodeDataEntry)
		{
			NodeDataEntry = &NodeData.AddDefaulted_GetRef();
			NodeDataEntry->Node = InputPinReference.OwningNode;
		}

		FPinData& PinDataEntry = NodeDataEntry->PinData.AddDefaulted_GetRef();
		PinDataEntry.Pin = InputPinReference.Pin;
		PinDataEntry.DataView = InInputDataViews[Index];
	}

	// Set the Node Instance Input Data Views with the corresponding Input Data Views
	// This does not guarantee that all the input pins have their data filled.
	// It could be that there's still data not ready yet.
	for (const FNodeData& NodeDataEntry : NodeData)
	{
		FDataLinkInputDataViewer& InputDataViewer = FindOrAddNodeInstance(NodeDataEntry.Node).InputDataViewer;

		// Copy the input data view to the node instance
		for (const FPinData& PinData : NodeDataEntry.PinData)
		{
			InputDataViewer.SetEntryData(*PinData.Pin, PinData.DataView);
		}
	}

	uint16 NodesExecuted = 0;

	// Execute nodes that have all their data ready to go
	for (const FNodeData& NodeInfo : NodeData)
	{
		FDataLinkNodeInstance& NodeInstance = GetNodeInstanceMutable(NodeInfo.Node);

		// don't execute nodes that don't have their input data views fully set yet
		if (NodeInstance.InputDataViewer.HasInvalidDataEntry())
		{
			continue;
		}

		if (!ExecuteNode(*NodeInfo.Node))
		{
			return false;
		}

		++NodesExecuted;
	}

	if (OutNodesExecuted)
	{
		*OutNodesExecuted = NodesExecuted;
	}
	return true;
}

bool FDataLinkExecutor::ExecuteNode(const UDataLinkNode& InNode)
{
	FDataLinkNodeInstance& NodeInstance = GetNodeInstanceMutable(&InNode);
	NodeInstance.Status = EDataLinkNodeStatus::Executing;
	InNode.Execute(*this);
	return true;
}

void FDataLinkExecutor::StopInternal()
{
	// Stop active node instances
	for (const TPair<TObjectPtr<const UDataLinkNode>, FDataLinkNodeInstance>& Pair : NodeInstanceMap)
	{
		if (Pair.Key && Pair.Value.Status == EDataLinkNodeStatus::Executing)
		{
			Pair.Key->Stop(*this);
		}
	}

	NodeInstanceMap.Reset();
	Sink.Reset();
}

FDataLinkNodeInstance& FDataLinkExecutor::FindOrAddNodeInstance(const UDataLinkNode* InNode)
{
	check(InNode);
	FDataLinkNodeInstance* NodeInstance = NodeInstanceMap.Find(InNode);
	if (!NodeInstance)
	{
		NodeInstance = &NodeInstanceMap.Add(InNode, FDataLinkNodeInstance(*InNode));
	}
	return *NodeInstance;
}

bool FDataLinkExecutor::ValidateNodeExecutionStatus(const UDataLinkNode* InNode)
{
	FDataLinkNodeInstance* const NodeInstance = FindNodeInstanceMutable(InNode);
	if (!ensureAlwaysMsgf(NodeInstance, TEXT("[%.*s] Invalid Node Status. No node instance data found!"), GetContextName().Len(), GetContextName().GetData()))
	{
		return false;
	}

	const bool bWasExecuting = NodeInstance->Status == EDataLinkNodeStatus::Executing;
	if (!ensureAlwaysMsgf(bWasExecuting, TEXT("[%.*s] Invalid Node Status. Node was not executing!"), GetContextName().Len(), GetContextName().GetData()))
	{
		return false;
	}

#if !NO_LOGGING
	if (UE::DataLink::Private::CVarDataLinkDebugNodeData.GetValueOnGameThread())
	{
		UE_LOGF(LogDataLink, Log, "[%.*ls] Node %ls Finished Execution."
			"\n-------------------------------------------------------------"
			"\nINPUT PINS:\n%ls"
			"\n-------------------------------------------------------------"
			"\nOUTPUT PINS:\n%ls"
			"\n-------------------------------------------------------------"
			, GetContextName().Len(), GetContextName().GetData()
			, *InNode->GetName()
			, *NodeInstance->GetInputDataViewer().ToDebugString()
			, *NodeInstance->GetOutputDataViewer().ToDebugString());
	}
#endif
	return true;
}

void FDataLinkExecutor::BroadcastFailure()
{
	BroadcastOutputData(FConstStructView());
	Stop(EDataLinkExecutionResult::Failed);
}

void FDataLinkExecutor::BroadcastOutputData(FConstStructView InOutputData)
{
	// Ensure executor was running
	if (!bRunning)
	{
		return;
	}

	for (UDataLinkProcessor* OutputProcessor : OutputProcessors)
	{
		OutputProcessor->ProcessOutput(*this, InOutputData);
	}

	OnOutputData.ExecuteIfBound(*this, InOutputData);

	if (!bPersistExecution)
	{
		Stop(EDataLinkExecutionResult::Succeeded);
	}

	// Reset on graph end
	bPersistExecution = false;
}
