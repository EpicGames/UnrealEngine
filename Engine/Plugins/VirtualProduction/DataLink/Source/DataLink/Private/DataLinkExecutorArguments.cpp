// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkExecutorArguments.h"
#include "DataLinkExecutor.h"

FDataLinkExecutorArguments::FDataLinkExecutorArguments(const FDataLinkInstance& InInstance)
	: Instance(InInstance)
{
}

FDataLinkExecutorArguments::FDataLinkExecutorArguments(FDataLinkInstance&& InInstance)
	: Instance(MoveTemp(InInstance))
{
}

#if WITH_DATALINK_CONTEXT
FDataLinkExecutorArguments&& FDataLinkExecutorArguments::SetContextName(const FString& InContextName)
{
	ContextName = InContextName;
	return MoveTemp(*this);
}

FDataLinkExecutorArguments&& FDataLinkExecutorArguments::SetContextName(FString&& InContextName)
{
	ContextName = MoveTemp(InContextName);
	return MoveTemp(*this);
}
#endif

FDataLinkExecutorArguments&& FDataLinkExecutorArguments::SetContextObject(UObject* InContextObject)
{
	ContextObject = InContextObject;
	return MoveTemp(*this);
}

FDataLinkExecutorArguments&& FDataLinkExecutorArguments::SetSink(const TSharedPtr<FDataLinkSink>& InSink)
{
	Sink = InSink;
	return MoveTemp(*this);
}

FDataLinkExecutorArguments&& FDataLinkExecutorArguments::SetOutputProcessors(TConstArrayView<UDataLinkProcessor*> InOutputProcessors)
{
	OutputProcessors = InOutputProcessors;
	return MoveTemp(*this);
}

FDataLinkExecutorArguments&& FDataLinkExecutorArguments::SetOnOutputData(FOnDataLinkOutputData&& InDelegate)
{
	OnOutputData = MoveTemp(InDelegate);
	return MoveTemp(*this);
}

FDataLinkExecutorArguments&& FDataLinkExecutorArguments::SetOnFinished(FOnDataLinkExecutionFinished&& InDelegate)
{
	OnExecutionFinished = MoveTemp(InDelegate);
	return MoveTemp(*this);
}
