// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkExecutorObject.h"
#include "DataLinkExecutor.h"
#include "DataLinkExecutorArguments.h"
#include "DataLinkUtils.h"

bool UDataLinkExecutorObject::IsRunning() const
{
	return Executor.IsValid() && Executor->IsRunning();
}

void UDataLinkExecutorObject::Run()
{
	Stop();

	Executor = FDataLinkExecutor::Create(FDataLinkExecutorArguments(DataLinkInstance)
#if WITH_DATALINK_CONTEXT
		.SetContextName(ContextName)
#endif
		.SetContextObject(GetOuter())
		.SetSink(UE::DataLink::TryGetSink(SinkProvider))
		.SetOnOutputData(FOnDataLinkOutputData::CreateUObject(this, &UDataLinkExecutorObject::NotifyOutputDataReceived))
		.SetOnFinished(FOnDataLinkExecutionFinished::CreateUObject(this, &UDataLinkExecutorObject::NotifyExecutionFinished)));

	Executor->Run();
}

void UDataLinkExecutorObject::Stop()
{
	if (Executor.IsValid())
	{
		Executor->Stop();
		Executor.Reset();
	}
}

void UDataLinkExecutorObject::NotifyOutputDataReceived(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView)
{
	OnOutputData.Broadcast(FInstancedStruct(InOutputDataView));
}

void UDataLinkExecutorObject::NotifyExecutionFinished(const FDataLinkExecutor& InExecutor, EDataLinkExecutionResult InExecutionResult)
{
	OnExecutionFinished.Broadcast(InExecutionResult);
}
