// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraph.h"
#include "DataLinkNode.h"
#include "DataLinkPinReference.h"
#include "StructUtils/StructView.h"

#if WITH_EDITOR
TMulticastDelegate<void(UDataLinkGraph*)> UDataLinkGraph::OnGraphCompiledDelegate;
#endif

int32 UDataLinkGraph::GetInputPinCount() const
{
	int32 InputPinCount = 0;

	ForEachInputPin([&InputPinCount](FDataLinkPinReference)
		{
			++InputPinCount;
			return true;
		});

	return InputPinCount;
}

TArray<FDataLinkPinReference> UDataLinkGraph::GetInputPins() const
{
	TArray<FDataLinkPinReference> InputPins;
	InputPins.Reserve(GetInputPinCount());

	ForEachInputPin(
		[&InputPins](FDataLinkPinReference InPinReference)
		{
			InputPins.Add(MoveTemp(InPinReference));
			return true;
		});

	return InputPins;
}

bool UDataLinkGraph::ForEachInputPin(TFunctionRef<bool(FDataLinkPinReference)> InFunction) const
{
	for (const UDataLinkNode* InputNode : InputNodes)
	{
		if (InputNode)
		{
			for (const FDataLinkPin& InputPin : InputNode->GetInputPins())
			{
				if (InputPin.LinkedNode == nullptr && !InFunction(FDataLinkPinReference(InputNode, &InputPin)))
				{
					return false;
				}
			}
		}
	}
	return true;
}
