// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkInputDataViewer.h"
#include "DataLinkPin.h"
#include "DataLinkUtils.h"

FString FDataLinkInputDataEntry::ToDebugString() const
{
	return Name.ToString() + TEXT(" : ") + UE::DataLink::StructViewToDebugString(DataView);
}

FDataLinkInputDataViewer::FDataLinkInputDataViewer(TConstArrayView<FDataLinkPin> InInputPins)
{
	DataEntries.Reserve(InInputPins.Num());

	// Initialize data entries with only name set
	for (const FDataLinkPin& InputPin : InInputPins)
	{
		FDataLinkInputDataEntry& DataEntry = DataEntries.AddDefaulted_GetRef();
		DataEntry.Name = InputPin.Name;
	}
}

FConstStructView FDataLinkInputDataViewer::Find(FName InInputName) const
{
	if (const FDataLinkInputDataEntry* FoundEntry = DataEntries.FindByKey(InInputName))
	{
		return FoundEntry->DataView;
	}
	return FConstStructView();
}

TConstArrayView<FDataLinkInputDataEntry> FDataLinkInputDataViewer::GetDataEntries() const
{
	return DataEntries;
}

FString FDataLinkInputDataViewer::ToDebugString() const
{
	return FString::JoinBy(DataEntries, TEXT("\n"),
		[](const FDataLinkInputDataEntry& InDataEntry)
		{
			return InDataEntry.ToDebugString();
		});
}

bool FDataLinkInputDataViewer::HasInvalidDataEntry() const
{
	for (const FDataLinkInputDataEntry& DataEntry : DataEntries)
	{
		if (!DataEntry.DataView.IsValid())
		{
			return true;
		}
	}
	return false;
}

void FDataLinkInputDataViewer::SetEntryData(const FDataLinkPin& InPin, FConstStructView InInputDataView)
{
	FDataLinkInputDataEntry* DataEntry = DataEntries.FindByKey(InPin.Name);
	check(DataEntry);
	DataEntry->DataView = InInputDataView;
}
