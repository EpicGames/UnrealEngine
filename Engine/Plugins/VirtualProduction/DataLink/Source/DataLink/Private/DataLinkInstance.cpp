// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkInstance.h"
#include "StructUtils/StructView.h"

bool FDataLinkInputData::SerializeFromMismatchedTag(const FPropertyTag& InPropertyTag, FStructuredArchive::FSlot InSlot)
{
	if (InPropertyTag.GetType().IsStruct(FInstancedStruct::StaticStruct()->GetFName()))
	{
		FInstancedStruct::StaticStruct()->SerializeItem(InSlot, &Data, nullptr);
		return true;
	}
	return false;
}

TArray<FConstStructView> FDataLinkInstance::GetInputDataViews() const
{
	TArray<FConstStructView> StructViews;
	StructViews.Reserve(InputData.Num());

	for (const FDataLinkInputData& InputDataEntry : InputData)
	{
		StructViews.Emplace(InputDataEntry.Data);
	}

	return StructViews;
}
