// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "DataLinkLog.h"

DEFINE_LOG_CATEGORY(LogDataLink);

IMPLEMENT_MODULE(FDefaultModuleImpl, DataLink)
