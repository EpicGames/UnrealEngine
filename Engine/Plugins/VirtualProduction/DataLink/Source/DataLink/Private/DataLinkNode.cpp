// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkNode.h"
#include "DataLinkEnums.h"
#include "DataLinkExecutor.h"
#include "DataLinkLog.h"
#include "DataLinkNodeMetadata.h"
#include "DataLinkPinBuilder.h"

void UDataLinkNode::Execute(FDataLinkExecutor& InExecutor) const
{
	const EDataLinkExecutionReply Reply = OnExecute(InExecutor);

	if (Reply == EDataLinkExecutionReply::Unhandled)
	{
		UE_LOGF(LogDataLink, Log, "[%ls] Node '%ls' did not handle execution.", InExecutor.GetContextName().GetData(), *GetName())
		InExecutor.Fail(this);
	}
}

void UDataLinkNode::Stop(const FDataLinkExecutor& InExecutor) const
{
	OnStop(InExecutor);
}

#if WITH_EDITOR
void UDataLinkNode::FixupNode()
{
	OnFixupNode();
}

void UDataLinkNode::BuildMetadata(FDataLinkNodeMetadata& OutMetadata) const
{
	OutMetadata
		.SetDisplayName(GetClass()->GetDisplayNameText())
		.SetTooltipText(GetClass()->GetToolTipText());

	OnBuildMetadata(OutMetadata);
}
#endif

void UDataLinkNode::BuildPins(TArray<FDataLinkPin>& OutInputPins, TArray<FDataLinkPin>& OutOutputPins) const
{
	FDataLinkPinBuilder InputBuilder(OutInputPins);
	FDataLinkPinBuilder OutputBuilder(OutOutputPins);
	OnBuildPins(InputBuilder, OutputBuilder);
}
