// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkNodeInstance.h"
#include "DataLinkNode.h"

FDataLinkNodeInstance::FDataLinkNodeInstance(const UDataLinkNode& InNode)
	: InputDataViewer(InNode.GetInputPins())
	, OutputDataViewer(InNode.GetOutputPins())
	, InstanceData(InNode.GetInstanceStruct())
{
}

void FDataLinkNodeInstance::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InstanceData.AddStructReferencedObjects(InCollector);
}
