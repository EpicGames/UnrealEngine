// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkOutputDataViewer.h"
#include "DataLinkPin.h"
#include "DataLinkUtils.h"

FStructView FDataLinkOutputDataEntry::GetDataView(const UScriptStruct* InDesiredStruct) const
{
	if (InDesiredStruct && OutputData.GetScriptStruct() != InDesiredStruct)
	{
		OutputData.InitializeAs(InDesiredStruct);
	}
	return FStructView(OutputData);
}

FString FDataLinkOutputDataEntry::ToDebugString() const
{
	return Name.ToString() + TEXT(" : ") + UE::DataLink::StructViewToDebugString(OutputData);
}

FDataLinkOutputDataViewer::FDataLinkOutputDataViewer(TConstArrayView<FDataLinkPin> InOutputPins)
{
	DataEntries.Reserve(InOutputPins.Num());

	// Initialize entries with only name set
	for (const FDataLinkPin& OutputPin : InOutputPins)
	{
		FDataLinkOutputDataEntry& DataEntry = DataEntries.AddDefaulted_GetRef();
		DataEntry.Name = OutputPin.Name;
	}
}

FStructView FDataLinkOutputDataViewer::Find(FName InOutputName, const UScriptStruct* InDesiredStruct) const
{
	if (const FDataLinkOutputDataEntry* FoundEntry = DataEntries.FindByKey(InOutputName))
	{
		return FoundEntry->GetDataView(InDesiredStruct);	
	}
	return FStructView();
}

FString FDataLinkOutputDataViewer::ToDebugString() const
{
	return FString::JoinBy(DataEntries, TEXT("\n"),
		[](const FDataLinkOutputDataEntry& InDataEntry)
		{
			return InDataEntry.ToDebugString();
		});
}
