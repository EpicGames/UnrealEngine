// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkPin.h"
#include "DataLinkNode.h"

FDataLinkPin::FDataLinkPin(FName InPinName)
	: Name(InPinName)
{
}

const FDataLinkPin* FDataLinkPin::GetLinkedInputPin() const
{
	if (!LinkedNode)
	{
		return nullptr;
	}

	TConstArrayView<FDataLinkPin> LinkedInputPins = LinkedNode->GetInputPins();
	if (LinkedInputPins.IsValidIndex(LinkedIndex))
	{
		return &LinkedInputPins[LinkedIndex];
	}

	return nullptr;
}

FText FDataLinkPin::GetDisplayName() const
{
	if (DisplayName.IsEmpty())
	{
		return FText::FromString(Name.ToString());
	}
	return DisplayName;
}
