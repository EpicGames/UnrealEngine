// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkPinBuilder.h"

FDataLinkPinParameters::FDataLinkPinParameters(FDataLinkPin& InPin)
	: Pin(InPin)
{
}

FDataLinkPinParameters& FDataLinkPinParameters::SetDisplayName(FText&& InPinDisplayName)
{
	Pin.DisplayName = MoveTemp(InPinDisplayName);
	return *this;
}

FDataLinkPinParameters& FDataLinkPinParameters::SetStruct(const UScriptStruct* InPinStruct)
{
	Pin.Struct = InPinStruct;
	return *this;
}

FDataLinkPinBuilder::FDataLinkPinBuilder(TArray<FDataLinkPin>& OutPins)
	: Pins(OutPins)
{
}

void FDataLinkPinBuilder::AddCapacity(int32 InNumToAdd)
{
	Pins.Reserve(Pins.Num() + InNumToAdd);
}

FDataLinkPinParameters FDataLinkPinBuilder::Add(FName InPinName)
{
	checkf(!Pins.Contains(InPinName), TEXT("Pin Name '%s' already exists! Should be unique"), *InPinName.ToString());
	return FDataLinkPinParameters(Pins.Emplace_GetRef(InPinName));
}
