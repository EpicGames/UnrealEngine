// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkProcessor.h"
#include "DataLinkExecutor.h"

void UDataLinkProcessor::Initialize(const FDataLinkExecutor& InExecutor)
{
	OnInitialize(InExecutor);
}

void UDataLinkProcessor::ProcessOutput(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView)
{
	OnProcessOutput(InExecutor, InOutputDataView);
}

void UDataLinkProcessor::Finalize(const FDataLinkExecutor& InExecutor, EDataLinkExecutionResult InExecutionResult)
{
	OnFinalize(InExecutor, InExecutionResult);
}
