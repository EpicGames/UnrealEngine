// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkProcessorBlueprint.h"
#include "DataLinkExecutor.h"
#include "Engine/World.h"

UObject* UDataLinkProcessorBlueprint::GetContextObject() const
{
	if (TSharedPtr<const FDataLinkExecutor> Executor = ExecutorWeak.Pin())
	{
		return Executor->GetContextObject();
	}
	return nullptr;
}

void UDataLinkProcessorBlueprint::OnInitialize(const FDataLinkExecutor& InExecutor)
{
	ExecutorWeak = InExecutor.AsWeak();
	Initialize();
}

void UDataLinkProcessorBlueprint::OnProcessOutput(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView)
{
	OutputData = InOutputDataView;
	ExecutorWeak = InExecutor.AsWeak();
	ProcessOutput(OutputData);
}

void UDataLinkProcessorBlueprint::OnFinalize(const FDataLinkExecutor& InExecutor, EDataLinkExecutionResult InExecutionResult)
{
	ExecutorWeak = InExecutor.AsWeak();
	Finalize(InExecutionResult);
}

UWorld* UDataLinkProcessorBlueprint::GetWorld() const
{
	if (UObject* Context = GetContextObject())
	{
		if (UWorld* World = Context->GetWorld())
		{
			return World;
		}
	}
	// Blueprint processors could use latent action nodes like 'Delay' that require a valid world from GetWorld().
	// If context object does not have a valid world, default to GWorld
	return GWorld;
}
