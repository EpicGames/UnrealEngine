// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkRequestProxy.h"
#include "Containers/Ticker.h"
#include "DataLinkEnums.h"
#include "DataLinkExecutor.h"
#include "DataLinkExecutorArguments.h"
#include "DataLinkUtils.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDataLinkRequestProxy* UDataLinkRequestProxy::CreateRequestProxy(FDataLinkInstance InDataLinkInstance, UObject* InExecutionContext, TScriptInterface<IDataLinkSinkProvider> InDataLinkSinkProvider)
{
	UDataLinkRequestProxy* RequestProxy = NewObject<UDataLinkRequestProxy>();
	RequestProxy->SetFlags(RF_StrongRefOnFrame);
	RequestProxy->ProcessRequest(MoveTemp(InDataLinkInstance), InExecutionContext, InDataLinkSinkProvider);
	return RequestProxy;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UDataLinkRequestProxy::ProcessRequest(FDataLinkInstance&& InDataLinkInstance, UObject* InExecutionContext, TScriptInterface<IDataLinkSinkProvider> InDataLinkSinkProvider)
{
	DataLinkExecutor = FDataLinkExecutor::Create(FDataLinkExecutorArguments(MoveTemp(InDataLinkInstance))
#if WITH_DATALINK_CONTEXT
		.SetContextName(GetName())
#endif
		.SetContextObject(InExecutionContext)
		.SetSink(UE::DataLink::TryGetSink(InDataLinkSinkProvider))
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		.SetOnOutputData(FOnDataLinkOutputData::CreateUObject(this, &UDataLinkRequestProxy::OnOutputDataReceived)));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Delay execution by 1 frame. This is so that even for executions that finish immediately,
	// the Async Call blueprint script still has chance to get the returned Proxy and listen to execution finished event
	// Note: a fix for this would be to have the Async Node create the object first, bind to the delegate, then execute a 'run' command
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this,
		[this](float) mutable -> bool 
		{
			if (ensureAlways(DataLinkExecutor.IsValid()))
			{
				DataLinkExecutor->Run();
			}
			// Fire once
			return false;
		}));
}

void UDataLinkRequestProxy::OnOutputDataReceived(const FDataLinkExecutor& InExecutor, FConstStructView InOutputData)
{
	FInstancedStruct OutputData(InOutputData);

	const EDataLinkExecutionResult ExecutionResult = InOutputData.IsValid()
		? EDataLinkExecutionResult::Succeeded
		: EDataLinkExecutionResult::Failed;

	// Allow Blueprint CallInEditor functions to handle then execution result
	FEditorScriptExecutionGuard ScriptGuard;
	OnOutputData.Broadcast(OutputData, ExecutionResult);
}
