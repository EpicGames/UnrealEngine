// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkSink.h"
#include "DataLinkNode.h"
#include "DataLinkNodeInstance.h"
#include "StructUtils/UserDefinedStruct.h"

namespace UE::DataLink::Private
{
	bool CompareDataEntries(const FDataLinkInputDataEntry& InDataEntry, const FDataLinkInputDataEntry& InOtherDataEntry)
	{
		if (InDataEntry.Name != InOtherDataEntry.Name)
		{
			return false;
		}

		const UScriptStruct* DataStruct = InDataEntry.DataView.GetScriptStruct();
		if (DataStruct != InOtherDataEntry.DataView.GetScriptStruct())
		{
			return false;
		}

		constexpr uint32 PortFlags = PPF_None;
		return !DataStruct || DataStruct->CompareScriptStruct(InDataEntry.DataView.GetMemory(), InOtherDataEntry.DataView.GetMemory(), PortFlags);
	}
}

FDataLinkSinkKey::FDataLinkSinkKey(TSubclassOf<UDataLinkNode> InNodeClass, const FDataLinkInputDataViewer& InInputDataViewer)
	: NodeClass(InNodeClass)
	, InputDataEntries(InInputDataViewer.GetDataEntries())
{
	CachedHash = GetTypeHash(NodeClass);

	for (const FDataLinkInputDataEntry& InputDataEntry : InputDataEntries)
	{
		const UScriptStruct* InputStruct = InputDataEntry.DataView.GetScriptStruct();

		CachedHash = HashCombineFast(CachedHash, GetTypeHash(InputDataEntry.Name));
		CachedHash = HashCombineFast(CachedHash, GetTypeHash(InputStruct));

		if (InputDataEntry.DataView.IsValid())
		{
			CachedHash = HashCombineFast(CachedHash, UE::StructUtils::GetStructHash64(InputDataEntry.DataView));
		}
	}
}

bool FDataLinkSinkKey::operator==(const FDataLinkSinkKey& InOther) const
{
	if (CachedHash != InOther.CachedHash)
	{
		return false;
	}

	if (NodeClass != InOther.NodeClass)
	{
		return false;
	}

	if (InputDataEntries.Num() != InOther.InputDataEntries.Num())
	{
		return false;
	}

	for (int32 Index = 0; Index < InputDataEntries.Num(); ++Index)
	{
		if (!UE::DataLink::Private::CompareDataEntries(InputDataEntries[Index], InOther.InputDataEntries[Index]))
		{
			return false;
		}
	}

	return true;
}

uint64 GetTypeHash(const FDataLinkSinkKey& InKey)
{
	return InKey.CachedHash;
}

FInstancedStruct& FDataLinkSink::FindOrAddCachedData(const FDataLinkNodeInstance& InNodeInstance, TOptional<const UScriptStruct*> InDesiredStruct)
{
	FInstancedStruct& InstancedStruct = CachedDataMap.FindOrAdd(InNodeInstance.GetSinkKey());
	if (InDesiredStruct.IsSet() && InstancedStruct.GetScriptStruct() != *InDesiredStruct)
	{
		InstancedStruct.InitializeAs(*InDesiredStruct);
	}
	return InstancedStruct;
}

void FDataLinkSink::AddStructReferencedObjects(FReferenceCollector& InCollector)
{
	for (TPair<FDataLinkSinkKey, FInstancedStruct>& Pair : CachedDataMap)
	{
		Pair.Value.AddStructReferencedObjects(InCollector);
	}
}
