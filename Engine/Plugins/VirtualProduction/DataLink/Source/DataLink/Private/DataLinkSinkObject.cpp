// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkSinkObject.h"
#include "DataLinkSink.h"

UDataLinkSinkObject::UDataLinkSinkObject()
{
	Sink = MakeShared<FDataLinkSink>();
}

void UDataLinkSinkObject::ResetSink()
{
	Sink = MakeShared<FDataLinkSink>();
}

TSharedPtr<FDataLinkSink> UDataLinkSinkObject::GetSink() const
{
	return Sink;
}

const UDataLinkSinkObject* UDataLinkSinkObject::GetSinkObject_Implementation() const
{
	return this;
}
