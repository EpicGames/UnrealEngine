// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkUtils.h"
#include "DataLinkGraph.h"
#include "DataLinkInstance.h"
#include "DataLinkLog.h"
#include "DataLinkPin.h"
#include "DataLinkPinReference.h"
#include "DataLinkSinkObject.h"
#include "Engine/Engine.h"
#include "IDataLinkSinkProvider.h"
#include "StructUtils/StructView.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/PropertyAccessUtil.h"

namespace UE::DataLink
{

bool CopyDataView(FStructView InDestView, FConstStructView InSourceView)
{
	if (!InDestView.IsValid() || !InSourceView.IsValid())
	{
		return false;
	}

	if (InDestView.GetScriptStruct() != InSourceView.GetScriptStruct())
	{
		return false;
	}

	InDestView.GetScriptStruct()->CopyScriptStruct(InDestView.GetMemory(), InSourceView.GetMemory());
	return true;
}

bool ReplaceObject(UObject*& InOutObject, UObject* InOuter, UClass* InClass)
{
	// Check if the object class already matches the new class
	if (InOutObject && InOutObject->GetClass() == InClass)
	{
		UE_LOGF(LogDataLink, Log, "ReplaceObject did not take place as '%ls' is already of class %ls."
			, *InOutObject->GetName()
			, *GetNameSafe(InClass));
		return false;
	}

	UObject* const OldObject = InOutObject;
	InOutObject = nullptr;

	// Save the current object name before renaming it
	const FName ObjectName = InOutObject ? InOutObject->GetFName() : NAME_None;

	// Discard current object
	if (InOutObject)
	{
		UObject* const NewOuter = GetTransientPackage();
		const FName UniqueName = MakeUniqueObjectName(NewOuter, InOutObject->GetClass(), *(TEXT("TRASH_") + InOutObject->GetName()));
		InOutObject->Rename(*UniqueName.ToString(), NewOuter, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		InOutObject->MarkAsGarbage();
		InOutObject = nullptr;
	}

	// Set the new class (only if new class is valid)
	// The operation is still considered valid if InClass is null. This 
	if (InClass)
	{
		const EObjectFlags ObjectFlags = InOuter ? InOuter->GetMaskedFlags(RF_PropagateToSubObjects) : RF_NoFlags;

		InOutObject = NewObject<UObject>(InOuter, InClass, ObjectName, ObjectFlags);

		if (OldObject && GEngine)
		{
			TMap<UObject*, UObject*> ReplacementMap;
			ReplacementMap.Add(OldObject, InOutObject);
			GEngine->NotifyToolsOfObjectReplacement(ReplacementMap);
		}
	}

	return true;
}

TSharedPtr<FDataLinkSink> TryGetSink(TScriptInterface<IDataLinkSinkProvider> InSinkProvider)
{
	if (IDataLinkSinkProvider* SinkProvider = InSinkProvider.GetInterface())
	{
		if (TSharedPtr<FDataLinkSink> Sink = SinkProvider->GetSink())
		{
			return Sink;
		}

		if (const UDataLinkSinkObject* SinkObject = SinkProvider->GetSinkObject())
		{
			return SinkObject->GetSink();
		}
	}
	else if (UObject* Object = InSinkProvider.GetObject())
	{
		if (Object->GetClass()->ImplementsInterface(UDataLinkSinkProvider::StaticClass()))
		{
			if (const UDataLinkSinkObject* SinkObject = IDataLinkSinkProvider::Execute_GetSinkObject(Object))
			{
				return SinkObject->GetSink();
			}
		}
	}

	return nullptr;
}

FConstPropertyView ResolveConstPropertyView(FConstStructView InBaseStructView, const FString& InPropertyPath, FString* OutError)
{
	if (!InBaseStructView.IsValid())
	{
		if (OutError)
		{
			*OutError = FString::Printf(TEXT("FindPropertyByPath Error. Invalid base struct data"));
		}
		return FConstPropertyView();
	}

	TArray<FString> PathSegments;
	InPropertyPath.ParseIntoArray(PathSegments, TEXT("."), /*bInCullEmpty*/true);
	if (PathSegments.IsEmpty())
	{
		if (OutError)
		{
			*OutError = FString::Printf(TEXT("FindPropertyByPath Error. Path '%s' is not properly formatted. No path segments (defined by delimiter '.') found."), *InPropertyPath);
		}
		return FConstPropertyView();
	}

	const UStruct* CurrentStruct = InBaseStructView.GetScriptStruct();

	FConstPropertyView Current;
	Current.Memory = InBaseStructView.GetMemory();

	for (const FString& PathSegment : PathSegments)
	{
		if (!CurrentStruct)
		{
			if (OutError)
			{
				*OutError = FString::Printf(TEXT("FindPropertyByPath Error. Path could not find property '%s' as previous property was not a struct. Path: %s")
					, *PathSegment
					, *InPropertyPath);
			}
			return FConstPropertyView();
		}

		Current.Property = PropertyAccessUtil::FindPropertyByName(*PathSegment, CurrentStruct);
		if (!Current.Property)
		{
			if (OutError)
			{
				*OutError = FString::Printf(TEXT("FindPropertyByPath Error. Property '%s' could not be found in struct '%s'. Path: %s.")
					, *PathSegment
					, *CurrentStruct->GetName()
					, *InPropertyPath);
			}
			return FConstPropertyView();
		}

		Current.Memory = Current.Property->ContainerPtrToValuePtr<uint8>(Current.Memory);

		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Current.Property))
		{
			CurrentStruct = StructProperty->Struct;
		}
		else
		{
			CurrentStruct = nullptr;
		}
	}

	return Current;
}

FPropertyView ResolvePropertyView(FStructView InBaseStructView, const FString& InPropertyPath, FString* OutError)
{
	FPropertyView Result;

	const FConstPropertyView ResolvedProperty = ResolveConstPropertyView(InBaseStructView, InPropertyPath, OutError);
	Result.Property = const_cast<FProperty*>(ResolvedProperty.Property);
	Result.Memory = const_cast<uint8*>(ResolvedProperty.Memory);

	return Result;
}

void SetInputData(UDataLinkGraph* InGraph, TArray<FDataLinkInputData>& OutInputData)
{
	if (!InGraph)
	{
		OutInputData.Reset();
		return;
	}

	const int32 InputPinCount = InGraph->GetInputPinCount();
	OutInputData.SetNum(InputPinCount);

	int32 Index = 0;
	InGraph->ForEachInputPin(
		[&OutInputData, &Index](const FDataLinkPinReference& InPinReference)->bool
		{
			FDataLinkInputData& InputEntry = OutInputData[Index];
			InputEntry.DisplayName = InPinReference.Pin->GetDisplayName();

			if (InputEntry.Data.GetScriptStruct() != InPinReference.Pin->Struct)
			{
				// Input Struct can be null, FInstancedStruct will just be reset
				InputEntry.Data.InitializeAs(InPinReference.Pin->Struct);
			}

			++Index;
			return true;
		});
}

FString StructViewToDebugString(FConstStructView InDataView)
{
	if (!InDataView.IsValid())
	{
		return FString();
	}

	FString DebugString;
	constexpr int32 PortFlags = PPF_PropertyWindow | PPF_BlueprintDebugView;
	InDataView.GetScriptStruct()->ExportText(DebugString, InDataView.GetMemory(), /*Defaults*/nullptr, /*OwnerObject*/nullptr, PortFlags, /*ExportRootScope*/nullptr);
	return DebugString;
}

} // UE::DataLink
