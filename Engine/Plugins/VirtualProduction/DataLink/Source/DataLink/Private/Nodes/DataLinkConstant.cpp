// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/DataLinkConstant.h"
#include "DataLinkExecutor.h"
#include "DataLinkLog.h"
#include "DataLinkNames.h"
#include "DataLinkNodeInstance.h"
#include "DataLinkNodeMetadata.h"
#include "DataLinkPinBuilder.h"
#include "DataLinkUtils.h"

void UDataLinkConstant::SetStruct(const UScriptStruct* InStructType)
{
	Instance.InitializeAs(InStructType);
}

#if WITH_EDITOR
void UDataLinkConstant::OnFixupNode()
{
	Super::OnFixupNode();

	if (!Instance.IsValid())
	{
		return;
	}

	// For Instanced Object properties, make sure the object is duplicated and under this node
	auto InstanceObjectsFunc = [This=this](const FObjectProperty* InObjectProperty, void* InValueAddress)
		{
			if (!InObjectProperty || !InObjectProperty->HasAllPropertyFlags(CPF_InstancedReference))
			{
				return;
			}

			// Only consider objects that should be outered to this constant node by verifying that its current outer is also a constant node.
			UObject* const TemplateObject = InObjectProperty->GetObjectPropertyValue(InValueAddress);
			if (!TemplateObject || !TemplateObject->GetTypedOuter<UDataLinkConstant>())
			{
				return;
			}

			FObjectDuplicationParameters Parameters = InitStaticDuplicateObjectParams(TemplateObject, This);
			Parameters.DestName = MakeUniqueObjectName(Parameters.DestOuter, Parameters.SourceObject->GetClass(), Parameters.SourceObject->GetFName());
			Parameters.FlagMask = RF_AllFlags & ~RF_DefaultSubObject;
			Parameters.PortFlags |= PPF_DuplicateVerbatim; // Skip resetting text IDs

			UObject* InstanceObject = StaticDuplicateObjectEx(Parameters);
			InObjectProperty->SetObjectPropertyValue(InValueAddress, InstanceObject);
		};

	Instance.GetScriptStruct()->Visit(Instance.GetMutableMemory(),
		[&InstanceObjectsFunc](const FPropertyVisitorContext& InContext)->EPropertyVisitorControlFlow
		{
			if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InContext.Path.Top().Property))
			{
				InstanceObjectsFunc(ObjectProperty, InContext.Data.PropertyData);

				// Do not iterate into object. Object properties that don't have a UObject parent are the only ones that could need fixing.
				return EPropertyVisitorControlFlow::StepOver;
			}
			return EPropertyVisitorControlFlow::StepInto;
		});
}

void UDataLinkConstant::OnBuildMetadata(FDataLinkNodeMetadata& Metadata) const
{
	Super::OnBuildMetadata(Metadata);

	if (const UScriptStruct* Struct = Instance.GetScriptStruct())
	{
		Metadata
			.SetDisplayName(Struct->GetDisplayNameText())
			.SetTooltipText(Struct->GetToolTipText());
	}

	if (!DisplayName.IsEmpty())
	{
		Metadata.SetDisplayName(DisplayName);
	}
}
#endif

void UDataLinkConstant::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	Super::OnBuildPins(Inputs, Outputs);

	if (const UScriptStruct* Struct = Instance.GetScriptStruct())
	{
		Outputs.Add(UE::DataLink::OutputDefault)
			.SetStruct(Struct);
	}
}

EDataLinkExecutionReply UDataLinkConstant::OnExecute(FDataLinkExecutor& InExecutor) const
{
	if (!Instance.IsValid())
	{
		UE_LOGF(LogDataLink, Error, "[%.*ls] DataLinkConstant failed. Instance is not valid"
			, InExecutor.GetContextName().Len(), InExecutor.GetContextName().GetData());
		return EDataLinkExecutionReply::Unhandled;
	}

	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

	const FStructView OutputDataView = OutputDataViewer.Find(UE::DataLink::OutputDefault, Instance.GetScriptStruct());
	if (!UE::DataLink::CopyDataView(OutputDataView, Instance))
	{
		UE_LOGF(LogDataLink, Error, "[%.*ls] DataLinkConstant failed. %ls does not match output struct %ls"
			, InExecutor.GetContextName().Len(), InExecutor.GetContextName().GetData()
			, *GetNameSafe(Instance.GetScriptStruct())
			, *GetNameSafe(OutputDataView.GetScriptStruct()));
		return EDataLinkExecutionReply::Unhandled;
	}

	InExecutor.Next(this);
	return EDataLinkExecutionReply::Handled;
}
