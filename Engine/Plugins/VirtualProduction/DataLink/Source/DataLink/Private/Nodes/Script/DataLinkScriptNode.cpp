// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Script/DataLinkScriptNode.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "DataLinkCoreTypes.h"
#include "DataLinkExecutor.h"
#include "Engine/World.h"

#define LOCTEXT_NAMESPACE "DataLinkScriptNode"

void UDataLinkScriptNode::Execute(const UDataLinkNode* InNode, FDataLinkExecutor& InExecutor)
{
	Node = InNode;
	ExecutorWeak = InExecutor.AsWeak();
	OnExecute();
}

void UDataLinkScriptNode::Stop()
{
	OnStop();
}

bool UDataLinkScriptNode::Succeed(const FInstancedStruct& OutputData, bool bPersistExecution)
{
	const TSharedPtr<FDataLinkExecutor> Executor = ExecutorWeak.Pin();
	if (!Executor.IsValid())
	{
		return false;
	}

	const UScriptStruct* OutputDataStruct = OutputData.GetScriptStruct();
	if (!OutputDataStruct || OutputPin.Struct != OutputDataStruct)
	{
		Executor->Fail(Node);
		return false;
	}

	const FDataLinkNodeInstance& NodeInstance = Executor->GetNodeInstance(Node);

	const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

	const FStructView OutputDataView = OutputDataViewer.Find(OutputPin.Name, OutputPin.Struct);
	if (!OutputDataView.IsValid())
	{
		Executor->Fail(Node);
		return false;
	}

	OutputDataStruct->CopyScriptStruct(OutputDataView.GetMemory(), OutputData.GetMemory());

	if (bPersistExecution)
	{
		Executor->NextPersist(Node);
	}
	else
	{
		Executor->Next(Node);
	}
	return true;
}

void UDataLinkScriptNode::Fail()
{
	if (TSharedPtr<FDataLinkExecutor> Executor = ExecutorWeak.Pin())
	{
		Executor->Fail(Node);
	}
}

bool UDataLinkScriptNode::GetInputData(FInstancedStruct& InputData, FName InputName) const
{
	if (TSharedPtr<FDataLinkExecutor> Executor = ExecutorWeak.Pin())
	{
		const FDataLinkInputDataViewer& InputDataViewer = Executor->GetNodeInstance(Node).GetInputDataViewer();
		const FConstStructView InputDataView = InputDataViewer.Find(InputName);
		if (InputDataView.IsValid())
		{
			InputData = FInstancedStruct(InputDataView);
			return true;
		}
	}
	return false;
}

UWorld* UDataLinkScriptNode::GetWorld() const
{
	if (UObject* Context = GetContextObject())
	{
		if (UWorld* World = Context->GetWorld())
		{
			return World;
		}
	}
	// Blueprint nodes could use latent action nodes like 'Delay' that require a valid world from GetWorld().
	// If context object does not have a valid world, default to GWorld
	return GWorld;
}

UObject* UDataLinkScriptNode::GetContextObject() const
{
	if (TSharedPtr<FDataLinkExecutor> Executor = ExecutorWeak.Pin())
	{
		return Executor->GetContextObject();
	}
	return nullptr;
}

DEFINE_FUNCTION(UDataLinkScriptNode::execSucceedWildcard)
{
	// Read Out Struct property
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	const FStructProperty* OutputStructProperty = CastField<FStructProperty>(Stack.MostRecentProperty);
	void* OutputStructAddress = Stack.MostRecentPropertyAddress;

	P_GET_UBOOL(bPersistExecution);

	P_FINISH;

	*(bool*)RESULT_PARAM = false;

	if (!OutputStructProperty || !OutputStructProperty->Struct || !OutputStructAddress)
	{
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, FBlueprintExceptionInfo(EBlueprintExceptionType::AbortExecution
			, LOCTEXT("InvalidOutputStructWarning", "Failed to resolve the Output Data on 'Finish Event'")));

		P_THIS->Fail();
		return;
	}

	if (P_THIS->OutputPin.Struct != OutputStructProperty->Struct)
	{
		P_THIS->Fail();
		return;
	}

	P_NATIVE_BEGIN;

	if (TSharedPtr<FDataLinkExecutor> Executor = P_THIS->ExecutorWeak.Pin())
	{
		const FDataLinkNodeInstance& NodeInstance = Executor->GetNodeInstance(P_THIS->Node);

		const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

		const FStructView OutputDataView = NodeInstance.GetOutputDataViewer().Find(P_THIS->OutputPin.Name, P_THIS->OutputPin.Struct);
		if (OutputDataView.IsValid())
		{
			OutputStructProperty->Struct->CopyScriptStruct(OutputDataView.GetMemory(), OutputStructAddress);

			if (bPersistExecution)
			{
				Executor->NextPersist(P_THIS->Node);
			}
			else
			{
				Executor->Next(P_THIS->Node);
			}

			*(bool*)RESULT_PARAM = true;
		}
		else
		{
			Executor->Fail(P_THIS->Node);
		}
	}

	P_NATIVE_END;
}

DEFINE_FUNCTION(UDataLinkScriptNode::execGetInputDataWildcard)
{
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	const FStructProperty* InputStructProperty = CastField<FStructProperty>(Stack.MostRecentProperty);
	void* InputStructAddress = Stack.MostRecentPropertyAddress;

	P_GET_PROPERTY(FNameProperty, InputName);

	P_FINISH;

	*(bool*)RESULT_PARAM = false;

	if (!InputStructProperty || !InputStructProperty->Struct || !InputStructAddress)
	{
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, FBlueprintExceptionInfo(EBlueprintExceptionType::AbortExecution
			, LOCTEXT("InvalidInputStructWarning", "Failed to resolve the Input Data on 'Get Input Data'")));
		return;
	}

	P_NATIVE_BEGIN;

	if (TSharedPtr<FDataLinkExecutor> Executor = P_THIS->ExecutorWeak.Pin())
	{
		const FDataLinkInputDataViewer& InputDataViewer = Executor->GetNodeInstance(P_THIS->Node).GetInputDataViewer();
		const FConstStructView InputDataView = InputDataViewer.Find(InputName);

		if (InputDataView.IsValid() && InputDataView.GetScriptStruct() == InputStructProperty->Struct)
		{
			InputStructProperty->Struct->CopyScriptStruct(InputStructAddress, InputDataView.GetMemory());
			*(bool*)RESULT_PARAM = true;
		}
	}

	P_NATIVE_END;
}

#undef LOCTEXT_NAMESPACE
