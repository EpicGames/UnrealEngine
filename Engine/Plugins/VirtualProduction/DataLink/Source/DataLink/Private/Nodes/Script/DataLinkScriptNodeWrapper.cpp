// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/Script/DataLinkScriptNodeWrapper.h"
#include "DataLinkExecutor.h"
#include "DataLinkLog.h"
#include "DataLinkNodeInstance.h"
#include "DataLinkNodeMetadata.h"
#include "DataLinkPinBuilder.h"
#include "DataLinkUtils.h"
#include "Nodes/Script/DataLinkScriptNode.h"
#include "UObject/Package.h"

UDataLinkScriptNodeWrapper::UDataLinkScriptNodeWrapper()
{
	InstanceStruct = FDataLinkScriptNodeInstance::StaticStruct();
}

void UDataLinkScriptNodeWrapper::SetNodeClass(TSubclassOf<UDataLinkScriptNode> InNodeClass)
{
	NodeClass = InNodeClass;

	UObject* Node = TemplateNode;
	if (UE::DataLink::ReplaceObject(Node, this, InNodeClass))
	{
		TemplateNode = Cast<UDataLinkScriptNode>(Node);
	}
}

#if WITH_EDITOR
void UDataLinkScriptNodeWrapper::OnBuildMetadata(FDataLinkNodeMetadata& Metadata) const
{
	Super::OnBuildMetadata(Metadata);

	if (NodeClass)
	{
		Metadata
			.SetDisplayName(NodeClass->GetDisplayNameText())
			.SetTooltipText(NodeClass->GetToolTipText());
	}
}
#endif

void UDataLinkScriptNodeWrapper::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	Super::OnBuildPins(Inputs, Outputs);

	if (TemplateNode)
	{
		Inputs.AddCapacity(TemplateNode->InputPins.Num());

		for (const FDataLinkScriptPin& Pin : TemplateNode->InputPins)
		{
			Inputs.Add(Pin.Name)
				.SetStruct(Pin.Struct);
		}

		Outputs.Add(TemplateNode->OutputPin.Name)
			.SetStruct(TemplateNode->OutputPin.Struct);
	}
}

EDataLinkExecutionReply UDataLinkScriptNodeWrapper::OnExecute(FDataLinkExecutor& InExecutor) const
{
	if (!NodeClass)
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstanceMutable(this);

	FDataLinkScriptNodeInstance& Instance = NodeInstance.GetInstanceDataMutable().Get<FDataLinkScriptNodeInstance>();
	Instance.Node = NewObject<UDataLinkScriptNode>(GetTransientPackage(), NodeClass, NAME_None, RF_Transient, TemplateNode);
	Instance.Node->Execute(this, InExecutor);

	return EDataLinkExecutionReply::Handled;
}

void UDataLinkScriptNodeWrapper::OnStop(const FDataLinkExecutor& InExecutor) const
{
	if (!NodeClass)
	{
		return;
	}

	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	const FDataLinkScriptNodeInstance& Instance = NodeInstance.GetInstanceData().Get<const FDataLinkScriptNodeInstance>();
	if (Instance.Node)
	{
		Instance.Node->Stop();
	}
}

UDataLinkScriptNode* UDataLinkScriptNodeWrapper::GetDefaultNode() const
{
	if (NodeClass)
	{
		return NodeClass->GetDefaultObject<UDataLinkScriptNode>();
	}
	return nullptr;
}
