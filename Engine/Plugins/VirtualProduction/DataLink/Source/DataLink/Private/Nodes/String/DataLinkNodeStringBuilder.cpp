// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/String/DataLinkNodeStringBuilder.h"
#include "DataLinkCoreTypes.h"
#include "DataLinkExecutor.h"
#include "DataLinkNames.h"
#include "DataLinkPinBuilder.h"
#include "Nodes/String/DataLinkStringBuilder.h"

void UDataLinkNodeStringBuilder::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	FDataLinkStringBuilder(Segments, Tokens)
		.BuildInputPins(Inputs);

	Outputs.Add(UE::DataLink::OutputDefault)
		.SetStruct<FDataLinkString>();
}

EDataLinkExecutionReply UDataLinkNodeStringBuilder::OnExecute(FDataLinkExecutor& InExecutor) const
{
	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

	FDataLinkString& OutputString = OutputDataViewer.Get<FDataLinkString>(UE::DataLink::OutputDefault);

	const FDataLinkStringBuilder StringBuilder(Segments, Tokens);
	if (StringBuilder.BuildString(NodeInstance.GetInputDataViewer(), OutputString.Value))
	{
		InExecutor.Next(this);
	}
	else
	{
		InExecutor.Fail(this);
	}

	return EDataLinkExecutionReply::Handled;
}

#if WITH_EDITOR
void UDataLinkNodeStringBuilder::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UDataLinkNodeStringBuilder, Segments))
	{
		FDataLinkStringBuilder::GatherTokens(Segments, Tokens);
	}
}
#endif
