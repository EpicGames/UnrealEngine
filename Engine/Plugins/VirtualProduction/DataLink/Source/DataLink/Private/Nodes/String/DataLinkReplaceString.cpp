// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/String/DataLinkReplaceString.h"
#include "DataLinkCoreTypes.h"
#include "DataLinkExecutor.h"
#include "DataLinkNames.h"
#include "DataLinkNodeInstance.h"
#include "DataLinkPinBuilder.h"
#include "Internationalization/Regex.h"

#define LOCTEXT_NAMESPACE "DataLinkRegexReplaceString"

namespace UE::DataLink::Private
{
	/** Allocator to use for Groups. Typical usage would be 3 or less capture groups */
	using FCaptureGroupAllocator = TInlineAllocator<3>;

	/** Finds the unique capture groups used in the given string */
	TArray<int32, FCaptureGroupAllocator> FindCaptureGroups(const FString& InString)
	{
		TArray<int32, FCaptureGroupAllocator> Groups;
		static const FRegexPattern GroupPattern(TEXT("\\$(\\d+)"));

		FRegexMatcher GroupMatcher(GroupPattern, InString);
		while (GroupMatcher.FindNext())
		{
			const FString GroupString = *GroupMatcher.GetCaptureGroup(1);

			FString::ElementType* End = nullptr;
			const int32 GroupNumber = FCString::Strtoi(*GroupString, &End, /*Base*/10);

			// If End matches string end() then operation was successful
			if (GroupNumber >= 0 && End == (*GroupString + GroupString.Len()))
			{
				Groups.AddUnique(GroupNumber);
			}
		}

		return Groups;
	}

	FString RegexReplace(const FString& InString, const FDataLinkReplaceStringEntry& InReplaceEntry)
	{
		if (InString.IsEmpty())
		{
			return FString();
		}

		FRegexMatcher Matcher(FRegexPattern(InReplaceEntry.Pattern), InString);

		const TArray<int32, FCaptureGroupAllocator> CaptureGroups = FindCaptureGroups(InReplaceEntry.Replacement);

		FString Result;

		// One past the last index that was processed in the original string
		int32 FirstUnprocessedIndex = 0;

		while (Matcher.FindNext())
		{
			const int32 MatchBegin = Matcher.GetMatchBeginning();
			const int32 MatchEnd = Matcher.GetMatchEnding();

			// Process the characters in the indices between the first unprocessed to match begin
			if (FirstUnprocessedIndex < MatchBegin)
			{
				Result += InString.Mid(FirstUnprocessedIndex, MatchBegin - FirstUnprocessedIndex);
			}

			// Match end is the first index past the last index in the match, so it becomes the first unprocessed index
			FirstUnprocessedIndex = MatchEnd;

			FString Replacement = InReplaceEntry.Replacement;
			for (int32 CaptureGroup : CaptureGroups)
			{
				if (Matcher.GetCaptureGroupBeginning(CaptureGroup) != INDEX_NONE)
				{
					Replacement.ReplaceInline(*FString::Printf(TEXT("$%i"), CaptureGroup), *Matcher.GetCaptureGroup(CaptureGroup));
				}
			}

			Result += MoveTemp(Replacement);
		}

		// Add the remaining unprocessed characters to the result
		if (FirstUnprocessedIndex < InString.Len())
		{
			Result += InString.RightChop(FirstUnprocessedIndex);
		}

		return Result;
	}

} // UE::DataLink::Private

void UDataLinkReplaceString::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	Super::OnBuildPins(Inputs, Outputs);

	Inputs.Add(UE::DataLink::InputDefault)
		.SetDisplayName(LOCTEXT("InputStringDisplayName", "Input String"))
		.SetStruct<FDataLinkString>();

	Inputs.Add(UE::DataLink::InputReplaceSettings)
		.SetDisplayName(LOCTEXT("ReplaceSettingsDisplayName", "Replace Settings"))
		.SetStruct<FDataLinkReplaceStringSettings>();

	Outputs.Add(UE::DataLink::OutputDefault)
		.SetDisplayName(LOCTEXT("OutputDisplayName", "Output"))
		.SetStruct<FDataLinkString>();
}

EDataLinkExecutionReply UDataLinkReplaceString::OnExecute(FDataLinkExecutor& InExecutor) const
{
	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	const FDataLinkInputDataViewer& InputDataViewer = NodeInstance.GetInputDataViewer();
	const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

	const FDataLinkReplaceStringSettings& ReplaceSettings = InputDataViewer.Get<FDataLinkReplaceStringSettings>(UE::DataLink::InputReplaceSettings);

	FString& Result = OutputDataViewer.Get<FDataLinkString>(UE::DataLink::OutputDefault).Value;
	Result = InputDataViewer.Get<FDataLinkString>(UE::DataLink::InputDefault).Value;

	for (const FDataLinkReplaceStringEntry& ReplaceEntry : ReplaceSettings.ReplaceEntries)
	{
		Result = UE::DataLink::Private::RegexReplace(Result, ReplaceEntry);
	}

	InExecutor.Next(this);
	return EDataLinkExecutionReply::Handled;
}

#undef LOCTEXT_NAMESPACE
