// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/String/DataLinkStringBuilder.h"
#include "DataLinkCoreTypes.h"
#include "DataLinkInputDataViewer.h"
#include "DataLinkPinBuilder.h"
#include "StructUtils/StructView.h"

namespace UE::DataLink::Private
{
	FStringView TryGetTokenName(const FStringView InString)
	{
		// String with just {} should not be valid. Should at least have something in between the brackets
		if (InString.Len() > 2
			&& InString.StartsWith(TEXT("{"))
			&& InString.EndsWith(TEXT("}")))
		{
			// Removes the { and } from the start and end
			return InString.Mid(1, InString.Len() - 2);
		}
		return FStringView();
	}
}

bool FDataLinkStringBuilder::BuildString(const FDataLinkInputDataViewer& InTokenValues, FString& InOutResult) const
{
	if (InTokenValues.Num() != Tokens.Num())
	{
		return false;
	}

	int32 ResultStringLength = 0;
	ForEachResolvedSegment(InTokenValues,
		[&ResultStringLength](FStringView InResolvedSegment)
		{
			ResultStringLength += InResolvedSegment.Len();
		});

	InOutResult.Reset(ResultStringLength);

	ForEachResolvedSegment(InTokenValues,
		[&InOutResult](FStringView InResolvedSegment)
		{
			InOutResult += InResolvedSegment;
		});

	return true;
}

void FDataLinkStringBuilder::BuildInputPins(FDataLinkPinBuilder& Inputs) const
{
	Inputs.AddCapacity(Tokens.Num());

	for (const FDataLinkStringBuilderToken& Token : Tokens)
	{
		Inputs.Add(Token.Name)
			.SetStruct<FDataLinkString>();
	}
}

void FDataLinkStringBuilder::GatherTokens(TConstArrayView<FString> InSegments, TArray<FDataLinkStringBuilderToken>& OutTokens)
{
	// Try to re-use the old token count if existing as an initial allocation capacity
	TMap<FStringView, int32> TokenIndexMap;
	TokenIndexMap.Reserve(OutTokens.Num());

	OutTokens.Reset();

	for (int32 SegmentIndex = 0; SegmentIndex < InSegments.Num(); ++SegmentIndex)
	{
		const FStringView TokenName = UE::DataLink::Private::TryGetTokenName(InSegments[SegmentIndex]);
		if (TokenName.IsEmpty())
		{
			continue;
		}

		if (int32* ExistingTokenIndex = TokenIndexMap.Find(TokenName))
		{
			FDataLinkStringBuilderToken& Token = OutTokens[*ExistingTokenIndex];
			checkSlow(Token.Name == TokenName);
			Token.Indices.Add(SegmentIndex);
		}
		else
		{
			const int32 TokenIndex = OutTokens.AddDefaulted();
			TokenIndexMap.Add(TokenName, TokenIndex);

			FDataLinkStringBuilderToken& Token = OutTokens[TokenIndex];
			Token.Name = FName(TokenName);
			Token.Indices.Add(SegmentIndex);
		}
	}
}

void FDataLinkStringBuilder::ForEachResolvedSegment(const FDataLinkInputDataViewer& InTokenValues, TFunctionRef<void(FStringView InResolvedSegment)> InFunction) const
{
	// At this point, these two are required to match in count
	check(Tokens.Num() == InTokenValues.Num());

	for (int32 SegmentIndex = 0; SegmentIndex < Segments.Num(); ++SegmentIndex)
	{
		const FDataLinkStringBuilderToken* Token = Tokens.FindByPredicate(
			[SegmentIndex](const FDataLinkStringBuilderToken& InToken)->bool
			{
				return InToken.Indices.Contains(SegmentIndex);
			});

		if (Token)
		{
			InFunction(InTokenValues.Get<FDataLinkString>(Token->Name).Value);
		}
		else
		{
			InFunction(Segments[SegmentIndex]);
		}
	}
}
