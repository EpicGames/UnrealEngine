// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "DataLinkCoreTypes.generated.h"

USTRUCT(BlueprintType)
struct FDataLinkString
{
	GENERATED_BODY()

	friend uint32 GetTypeHash(const FDataLinkString& InString)
	{
		return GetTypeHash(InString.Value);
	}

	bool operator==(const FDataLinkString& InString) const
	{
		return Value == InString.Value;
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link")
	FString Value;
};

template<>
struct TStructOpsTypeTraits<FDataLinkString> : public TStructOpsTypeTraitsBase2<FDataLinkString>
{
	enum
	{
		WithIdenticalViaEquality = true
	};
};
