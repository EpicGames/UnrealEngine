// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"

#define UE_API DATALINK_API

namespace UE::DataLink
{
	enum class EGraphCompileStatus : uint8;
}

class FDataLinkExecutor;
class UDataLinkGraph;
enum class EDataLinkExecutionResult : uint8;
struct FConstStructView;

DECLARE_DELEGATE_TwoParams(FOnDataLinkOutputData, const FDataLinkExecutor&, FConstStructView);
DECLARE_DELEGATE_TwoParams(FOnDataLinkExecutionFinished, const FDataLinkExecutor&, EDataLinkExecutionResult);

namespace UE::DataLink
{
	DECLARE_DELEGATE_RetVal_OneParam(EGraphCompileStatus, FOnRequestGraphCompilation, UDataLinkGraph*);

	/** Delegate to request to the editor module to compile a data link graph */
	extern UE_API FOnRequestGraphCompilation OnRequestCompilation;

} // UE::DataLink

#undef UE_API
