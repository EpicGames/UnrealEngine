// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "DataLinkEnums.generated.h"

namespace UE::DataLink
{
	enum class EGraphCompileStatus : uint8
	{
		Unknown,
		Warning,
		Error,
		UpToDate,
	};

} // UE::DataLink

UENUM()
enum class EDataLinkExecutionResult : uint8
{
	Failed,
	Succeeded,
};

UENUM()
enum class EDataLinkExecutionReply : uint8
{
	Unhandled,
	Handled,
};
