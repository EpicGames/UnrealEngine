// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkDelegates.h"
#include "DataLinkEnums.h"
#include "DataLinkInstance.h"
#include "DataLinkNodeInstance.h"
#include "DataLinkSink.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/GCObject.h"

class FDataLinkExecutorArguments;
class UDataLinkGraph;
class UDataLinkNode;
class UDataLinkProcessor;
struct FDataLinkPinReference;

/**
 * Executes a Data Link graph with custom instance data.
 * @see FDataLinkExecutor::Run
 */
class FDataLinkExecutor : public FGCObject, public TSharedFromThis<FDataLinkExecutor>
{
	friend class FDataLinkExecutorArguments;

	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	DATALINK_API static TSharedPtr<FDataLinkExecutor> Create(FDataLinkExecutorArguments&& InArgs);

	explicit FDataLinkExecutor(FPrivateToken);

	virtual ~FDataLinkExecutor() override;

	/** Gets the unique identifier for this Executor */
	DATALINK_API const FGuid& GetExecutorId() const;

	/**
	 * Gets a brief description of the context for debugging purposes.
	 * Returns a valid context name when WITH_DATALINK_CONTEXT is true.
	 * Note: WITH_DATALINK_CONTEXT is true in builds with logging enabled.
	 */
	DATALINK_API FStringView GetContextName() const;

	/** Gets the object that is responsible for this execution */
	DATALINK_API UObject* GetContextObject() const;

	bool IsRunning() const
	{
		return bRunning;
	}

	/** Called to start the graph execution */
	DATALINK_API void Run();

	/** Called to stop graph execution */
	DATALINK_API void Stop(EDataLinkExecutionResult InExecutionResult = EDataLinkExecutionResult::Succeeded);

	/** Retrieves the node instance data of this execution for the given node */
	DATALINK_API const FDataLinkNodeInstance& GetNodeInstance(const UDataLinkNode* InNode) const;

	/** Retrieves the node instance data (mutable) of this execution for the given node */
	DATALINK_API FDataLinkNodeInstance& GetNodeInstanceMutable(const UDataLinkNode* InNode);

	/** Finds the node instance data (mutable) of this execution for the given node, or null if already removed */
	DATALINK_API FDataLinkNodeInstance* FindNodeInstanceMutable(const UDataLinkNode* InNode);

	/**
	 * Called when a node has received data successfully and is ready to pass it to the next node
	 * If this is called from the output node, it will broadcast this data to the listeners instead
	 * @param InNode the node that received this data
	 */
	DATALINK_API void Next(const UDataLinkNode* InNode);

	/**
	 * Used by Nodes that keep receiving data (instead of being a one-off) to keep execution active.
	 * Called when a node has received data successfully and is ready to pass it to the next node
	 * If this is called from the output node, it will broadcast this data to the listeners instead.
	 * @param InNode the node that received this data
	 */
	DATALINK_API void NextPersist(const UDataLinkNode* InNode);

	/**
	 * Called when a node has failed to receive data and produce a valid output
	 * This stops the entire graph execution
	 * @param InNode the node that failed
	 */
	DATALINK_API void Fail(const UDataLinkNode* InNode);

private:
	/** Called when running to ensure all the parameters are properly set. Returns true if run can be done, false otherwise */
	bool ValidateRun();

	/** Called to ensure the graph's output node is in a valid state. Returns true if run can be done, false otherwise */
	bool ValidateGraphOutputNode();

	/** Called in Pin execution to make sure the pins and data views match in compatibility */
	bool ValidateInputPins(TConstArrayView<FDataLinkPinReference> InInputPins, TConstArrayView<FConstStructView> InInputDataViews) const;

	/** Called to execute the graph's input nodes */
	bool ExecuteEntryNodes();

	/**
	 * Called to execute the given input pins with the input data
	 * @param InInputPins the input pins to execute (can belong to different nodes)
	 * @param InInputDataViews the matching data views for each input pin
	 * @param OutNodesExecuted the number of nodes that got executed
	 * @return true if the input pins were processed without error
	 */
	bool ExecuteInputPins(TConstArrayView<FDataLinkPinReference> InInputPins, TConstArrayView<FConstStructView> InInputDataViews, uint16* OutNodesExecuted = nullptr);

	bool ExecuteNode(const UDataLinkNode& InNode);

	/** Called to stop graph execution */
	void StopInternal();

	FDataLinkNodeInstance& FindOrAddNodeInstance(const UDataLinkNode* InNode);

	/**
	 * Called when a node has executed and finished its logic.
	 * Checks that the node instance is valid and executing
	 */
	bool ValidateNodeExecutionStatus(const UDataLinkNode* InNode);

	/** Called when there was a failure in the graph execution */
	void BroadcastFailure();

	/** Called when the entire graph execution has completed */
	void BroadcastOutputData(FConstStructView InOutputData);

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

	/** Unique Identifier for this Executor */
	const FGuid ExecutorId;

#if WITH_DATALINK_CONTEXT
	/** Context string for additional information when logging */
	FString ContextName;
#endif

	/** Object responsible for this execution */
	TObjectPtr<UObject> ContextObject;

	/** The data link instance of this execution */
	FDataLinkInstance Instance;

	/** Sink where all the data for this execution is queried and stored */
	TSharedPtr<FDataLinkSink> Sink;

	/** Container for all the output processors in this execution */
	TArray<TObjectPtr<UDataLinkProcessor>> OutputProcessors;

	/** Delegate to call when the data link outputs data */
	FOnDataLinkOutputData OnOutputData;

	/** Delegate to call when the data link execution finishes */
	FOnDataLinkExecutionFinished OnExecutionFinished;

	/** Map of a Node to its Instance (Input, Output and Instance Data) for this execution */
	mutable TMap<TObjectPtr<const UDataLinkNode>, FDataLinkNodeInstance> NodeInstanceMap;

	/** Whether to keep execution running upon data broadcast, or finish */
	bool bPersistExecution = false;

	bool bRunning = false;
};
