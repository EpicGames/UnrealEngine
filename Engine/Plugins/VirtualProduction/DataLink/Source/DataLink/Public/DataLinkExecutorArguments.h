// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "DataLinkDelegates.h"
#include "DataLinkInstance.h"
#include "StructUtils/StructView.h"

class UDataLinkGraph;
class UDataLinkProcessor;
struct FDataLinkSink;

class FDataLinkExecutorArguments
{
	friend class FDataLinkExecutor;

public:
	DATALINK_API FDataLinkExecutorArguments(const FDataLinkInstance& InInstance);

	DATALINK_API FDataLinkExecutorArguments(FDataLinkInstance&& InInstance);

#if WITH_DATALINK_CONTEXT
	/** Added context information of the execution */
	DATALINK_API FDataLinkExecutorArguments&& SetContextName(const FString& InContextName);
	/** Added context information of the execution */
	DATALINK_API FDataLinkExecutorArguments&& SetContextName(FString&& InContextName);
#endif

	/** Sets the object responsible for this execution */
	DATALINK_API FDataLinkExecutorArguments&& SetContextObject(UObject* InContextObject);

	/** Sets the Sink to use. If unset, the Executor will create its own temporary sink */
	DATALINK_API FDataLinkExecutorArguments&& SetSink(const TSharedPtr<FDataLinkSink>& InSink);

	/** Sets the output processors */
	DATALINK_API FDataLinkExecutorArguments&& SetOutputProcessors(TConstArrayView<UDataLinkProcessor*> InOutputProcessors);

	/** The delegate to call when the execution outputs data */
	DATALINK_API FDataLinkExecutorArguments&& SetOnOutputData(FOnDataLinkOutputData&& InDelegate);

	/** The delegate to call when the execution finishes */
	DATALINK_API FDataLinkExecutorArguments&& SetOnFinished(FOnDataLinkExecutionFinished&& InDelegate);

private:
	FDataLinkInstance Instance;

#if WITH_DATALINK_CONTEXT
	FString ContextName;
#endif

	TObjectPtr<UObject> ContextObject;

	TSharedPtr<FDataLinkSink> Sink;

	/** The output processors to use for the execution */
	TArray<TObjectPtr<UDataLinkProcessor>> OutputProcessors;

	/** Delegate called when data link execution has output data ready */
	FOnDataLinkOutputData OnOutputData;

	/** Delegate called when data link execution has finished completely */
	FOnDataLinkExecutionFinished OnExecutionFinished;
};
