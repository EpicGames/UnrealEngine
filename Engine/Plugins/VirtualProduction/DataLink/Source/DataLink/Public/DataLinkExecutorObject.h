// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkDelegates.h"
#include "DataLinkInstance.h"
#include "UObject/Object.h"
#include "DataLinkExecutorObject.generated.h"

#define UE_API DATALINK_API

class IDataLinkSinkProvider;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FReceiveDataLinkOutputData, FInstancedStruct, OutputData);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FReceiveDataLinkExecutionFinished, EDataLinkExecutionResult, Result);

/** UObject wrapper for Data Link Executor for Blueprint usage */
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UDataLinkExecutorObject : public UObject
{
	GENERATED_BODY()

	/** Returns whether there's an active data link execution happening */
	UFUNCTION(BlueprintPure, Category="Data Link")
	UE_API bool IsRunning() const;

	/** Executes the Data Link Instance, stopping the existing active execution if any */
	UFUNCTION(BlueprintCallable, Category="Data Link")
	UE_API void Run();

	/** Stops an existing data link execution if active */
	UFUNCTION(BlueprintCallable, Category="Data Link")
	UE_API void Stop();

protected:
	/** Called when data has been received. Calls the OnOutputData delegate */
	void NotifyOutputDataReceived(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView);

	/** Called when execution has finished. Calls the OnExecutionFinished delegate */
	void NotifyExecutionFinished(const FDataLinkExecutor& InExecutor, EDataLinkExecutionResult InExecutionResult);

	/** Data link graph and input data to execute */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link", meta=(ExposeOnSpawn="true"))
	FDataLinkInstance DataLinkInstance;

	/** Additional context to identify the data link execution */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link", meta=(ExposeOnSpawn="true"), AdvancedDisplay)
	FString ContextName;

	/** Optional: Sink to use for cross execution storage */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link", meta=(ExposeOnSpawn="true"), AdvancedDisplay)
	TScriptInterface<IDataLinkSinkProvider> SinkProvider;

	/** Delegate called when data link execution has output data ready */
	UPROPERTY(BlueprintAssignable, Category="Data Link", meta=(HideInDetailPanel))
	FReceiveDataLinkOutputData OnOutputData;

	/** Delegate called when data link execution has finished completely */
	UPROPERTY(BlueprintAssignable, Category="Data Link", meta=(HideInDetailPanel))
	FReceiveDataLinkExecutionFinished OnExecutionFinished;

	/** The underlying executor that this object wraps */
	TSharedPtr<FDataLinkExecutor> Executor;
};

#undef UE_API
