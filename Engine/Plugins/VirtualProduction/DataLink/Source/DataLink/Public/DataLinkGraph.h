// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "UObject/Object.h"
#include "DataLinkGraph.generated.h"

#define UE_API DATALINK_API

class UDataLinkNode;
struct FConstStructView;
struct FDataLinkPinReference;

#if WITH_EDITOR
class UEdGraph;
#endif

UCLASS(MinimalAPI, BlueprintType, DisplayName="Motion Design Data Link Graph")
class UDataLinkGraph : public UObject
{
	GENERATED_BODY()

	friend class FDataLinkGraphCompiler;
	friend class UDataLinkGraphFactory;

public:
	/** Counts the number of input pins within the input node list that are not linked to other nodes */
	UE_API int32 GetInputPinCount() const;

	/** Retrieves the Input Pins from the input nodes that are not linked to other nodes */
	UE_API TArray<FDataLinkPinReference> GetInputPins() const;

	/** Iterates the input pins from the input nodes that are not linked to other nodes. These serve as the 'input pin' for the graph */
	UE_API bool ForEachInputPin(TFunctionRef<bool(FDataLinkPinReference)> InFunction) const;

	TConstArrayView<const UDataLinkNode*> GetInputNodes() const
	{
		return InputNodes;
	}

	const UDataLinkNode* GetOutputNode() const
	{
		return OutputNode;
	}

	FName GetOutputPinName() const
	{
		return OutputPinName;
	}

#if WITH_EDITOR
	UEdGraph* GetEdGraph() const
	{
		return EdGraph;
	}

	static TMulticastDelegateRegistration<void(UDataLinkGraph*)>& OnGraphCompiled()
	{
		return OnGraphCompiledDelegate;
	}
#endif

private:
#if WITH_EDITOR
	UE_API static TMulticastDelegate<void(UDataLinkGraph*)> OnGraphCompiledDelegate;
#endif

	/** All the compiled nodes present in this graph */
	UPROPERTY()
	TArray<TObjectPtr<UDataLinkNode>> Nodes;

	/** The nodes that the graph starts off with */
	UPROPERTY()
	TArray<TObjectPtr<UDataLinkNode>> InputNodes;

	/** The node that provides the result data */
	UPROPERTY()
	TObjectPtr<UDataLinkNode> OutputNode;

	/** Output Node Pin name that provides the result data */
	UPROPERTY()
	FName OutputPinName;

#if WITH_EDITORONLY_DATA
	/** EdGraph used to compile the nodes in this graph */
	UPROPERTY()
	TObjectPtr<UEdGraph> EdGraph;
#endif
};

#undef UE_API
