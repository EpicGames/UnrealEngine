// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructView.h"

#define UE_API DATALINK_API

struct FDataLinkInputDataEntry
{
	bool operator==(FName InInputName) const
	{
		return Name == InInputName;
	}

	/** Converts the input data to a string for debugging purposes */
	FString ToDebugString() const;

	/** Name of the Input Data */
	FName Name;

	/** View to the Input Data */
	FConstStructView DataView;
};

struct FDataLinkPin;

class FDataLinkInputDataViewer
{
	friend class FDataLinkExecutor;

public:
	explicit FDataLinkInputDataViewer(TConstArrayView<FDataLinkPin> InInputPins);

	UE_API FConstStructView Find(FName InInputName) const;

	int32 Num() const
	{
		return DataEntries.Num();
	}

	template<typename T>
	const T& Get(FName InInputName) const
	{
		return Find(InInputName).Get<const T>();
	}

	TConstArrayView<FDataLinkInputDataEntry> GetDataEntries() const;

	/** Converts the output data to a string for debugging purposes */
	UE_API FString ToDebugString() const;
	
private:
	bool HasInvalidDataEntry() const;

	void SetEntryData(const FDataLinkPin& InPin, FConstStructView InInputDataView);

	TArray<FDataLinkInputDataEntry> DataEntries;
};

#undef UE_API
