// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "StructUtils/InstancedStruct.h"
#include "DataLinkInstance.generated.h"

#define UE_API DATALINK_API

class UDataLinkGraph;

/** Describes an Input Entry to the Data Link */
USTRUCT(BlueprintType)
struct FDataLinkInputData
{
	GENERATED_BODY()

	bool SerializeFromMismatchedTag(const FPropertyTag& InPropertyTag, FStructuredArchive::FSlot InSlot);

	/** The display name of the input data */
	UPROPERTY(VisibleAnywhere, Category="Data Link")
	FText DisplayName;

	/** The input data to feed into the data link graph */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link", meta=(StructTypeConst))
	FInstancedStruct Data;
};

template<>
struct TStructOpsTypeTraits<FDataLinkInputData> : public TStructOpsTypeTraitsBase2<FDataLinkInputData>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

/** Instance of a data link to be executed */
USTRUCT(BlueprintType)
struct FDataLinkInstance
{
	GENERATED_BODY()

	/** Converts the Input Data into an array of Const Struct Views */
	UE_API TArray<FConstStructView> GetInputDataViews() const;

	/** The data link graph to execute */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link")
	TObjectPtr<UDataLinkGraph> DataLinkGraph;

	/** The initial input data to feed into the data link graph */
	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintReadWrite, Category="Data Link", meta=(EditFixedOrder))
	TArray<FDataLinkInputData> InputData;
};

#undef UE_API
