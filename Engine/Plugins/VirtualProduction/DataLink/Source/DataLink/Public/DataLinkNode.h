// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkEnums.h"
#include "DataLinkPin.h"
#include "UObject/Object.h"
#include "DataLinkNode.generated.h"

#define UE_API DATALINK_API

class FDataLinkExecutor;
struct FDataLinkPinBuilder;

#if WITH_EDITOR
struct FDataLinkNodeMetadata;
#endif

/**
 * Base class for nodes in a data link graph, that handles the logic of the node.
 * The instance data of the node can be gotten through the FDataLinkExecutor.
 * Each node consists of an Input struct, and an Output struct.
 * A data link node can be connected to another so long as the output struct of one is compatible with the input struct of the other.
 */
UCLASS(Abstract, MinimalAPI)
class UDataLinkNode : public UObject
{
	friend class FDataLinkGraphCompiler;

	GENERATED_BODY()

public:
	void Execute(FDataLinkExecutor& InExecutor) const;

	/** Called when the node is still executing and is forced to stop execution and clean up its resources */
	void Stop(const FDataLinkExecutor& InExecutor) const;

	TConstArrayView<FDataLinkPin> GetInputPins() const
	{
		return InputPins;
	}

	TConstArrayView<FDataLinkPin> GetOutputPins() const
	{
		return OutputPins;
	}

	const UScriptStruct* GetInstanceStruct() const
	{
		return InstanceStruct;
	}

#if WITH_EDITOR
	/** Called when the node has been compiled to fix data to be valid to use at runtime */
	UE_API void FixupNode();
	UE_API void BuildMetadata(FDataLinkNodeMetadata& OutMetadata) const;
#endif

	UE_API void BuildPins(TArray<FDataLinkPin>& OutInputPins, TArray<FDataLinkPin>& OutOutputPins) const;

protected:
#if WITH_EDITOR
	virtual void OnFixupNode()
	{
	}

	/** Call to retrieve the metadata for this node */
	virtual void OnBuildMetadata(FDataLinkNodeMetadata& Metadata) const
	{
	}
#endif

	/** Call to setup the pins required by this node */
	virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
	{
	}

	/**
	 * Execution logic of the node
	 * @return whether the implementation handled the execution (regardless if it has finished or not)
	 */
	virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	/** Called when the node is still executing and is forced to stop execution and clean up its resources */
	virtual void OnStop(const FDataLinkExecutor& InExecutor) const
	{
	}

	/** Optional instance data struct for data outside input and output */
	UPROPERTY()
	TObjectPtr<const UScriptStruct> InstanceStruct;

private:
	/** Input Pins to start the execution on this Data Link node */
	UPROPERTY()
	TArray<FDataLinkPin> InputPins;

	/** Output pins that feed the result of this node's execution */
	UPROPERTY()
	TArray<FDataLinkPin> OutputPins;
};

#undef UE_API
