// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkInputDataViewer.h"
#include "DataLinkOutputDataViewer.h"
#include "DataLinkSink.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"

class FReferenceCollector;

enum class EDataLinkNodeStatus : uint8
{
	NotStarted,
	Executing,
};

struct FDataLinkNodeInstance
{
	friend class FDataLinkExecutor;

	explicit FDataLinkNodeInstance(const UDataLinkNode& InNode);

	void AddReferencedObjects(FReferenceCollector& InCollector);

	const FDataLinkInputDataViewer& GetInputDataViewer() const
	{
		return InputDataViewer;
	}

	const FDataLinkOutputDataViewer& GetOutputDataViewer() const
	{
		return OutputDataViewer;
	}

	FConstStructView GetInstanceData() const
	{
		return InstanceData;
	}

	FStructView GetInstanceDataMutable()
	{
		return InstanceData;
	}

	const FDataLinkSinkKey& GetSinkKey() const
	{
		return SinkKey;
	}

private:
	/**
	 * The Sink Key for this Node Instance
	 * Saved here to avoid recreating if it needs to be re-used.
	 */
	FDataLinkSinkKey SinkKey;

	/** Views of the input data, matching the Node's Input Pins */
	FDataLinkInputDataViewer InputDataViewer;

	/** Views of the output data, matching the Node's Output Pins */
	FDataLinkOutputDataViewer OutputDataViewer;

	/**
	 * Optional data within the node instanced for every execution.
	 * This is used to store data outside of the input/output data.
	 */
	FInstancedStruct InstanceData;

	/** Current Status of a Node */
	EDataLinkNodeStatus Status = EDataLinkNodeStatus::NotStarted;
};
