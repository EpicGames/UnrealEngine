// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Internationalization/Text.h"
#endif

#if WITH_EDITOR
/** Metadata of a Node for Editor-side representation. */
struct FDataLinkNodeMetadata
{
	FDataLinkNodeMetadata& SetDisplayName(const FText& InDisplayName)
	{
		DisplayName = InDisplayName;
		return *this;
	}

	FDataLinkNodeMetadata& SetTooltipText(const FText& InDescription)
	{
		TooltipText = InDescription;
		return *this;
	}

	const FText& GetDisplayName() const
	{
		return DisplayName;
	}

	const FText& GetTooltipText() const
	{
		return TooltipText;
	}

private:
	/** Display name of the Node */
	FText DisplayName;

	/** Description for the node. Used for the Tooltip text of nodes */
	FText TooltipText;
};
#endif
