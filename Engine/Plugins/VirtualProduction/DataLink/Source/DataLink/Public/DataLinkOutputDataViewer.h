// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructView.h"

#define UE_API DATALINK_API

struct FDataLinkPin;

struct FDataLinkOutputDataEntry
{
	bool operator==(FName InOutputName) const
	{
		return Name == InOutputName;
	}

	/** Name of the Output Data */
	FName Name;

	FStructView GetDataView(const UScriptStruct* InDesiredStruct) const;

	/** Converts the output data to a string for debugging purposes */
	FString ToDebugString() const;

private:
	/** Output Data instantiated */
	mutable FInstancedStruct OutputData;
};

class FDataLinkOutputDataViewer
{
public:
	explicit FDataLinkOutputDataViewer(TConstArrayView<FDataLinkPin> InOutputPins);

	UE_API FStructView Find(FName InOutputName, const UScriptStruct* InDesiredStruct = nullptr) const;

	int32 Num() const
	{
		return DataEntries.Num();
	}

	template<typename T>
	T& Get(FName InOutputName) const
	{
		return this->Find(InOutputName, T::StaticStruct()).template Get<T>();
	}

	UE_API FString ToDebugString() const;

private:
	TArray<FDataLinkOutputDataEntry> DataEntries;
};

#undef UE_API
