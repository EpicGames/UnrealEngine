// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "DataLinkPin.generated.h"

class UDataLinkNode;

USTRUCT()
struct FDataLinkPin
{
	GENERATED_BODY()

	FDataLinkPin() = default;

	DATALINK_API explicit FDataLinkPin(FName InPinName);

	DATALINK_API const FDataLinkPin* GetLinkedInputPin() const;

	DATALINK_API FText GetDisplayName() const;

	bool operator==(const FName InName) const
	{
		return Name == InName;
	}

	/** Unique name for the Pin */
	UPROPERTY()
	FName Name;

	/** Display name of the Pin */
	UPROPERTY()
	FText DisplayName;

	/** Struct type of the Pin */
	UPROPERTY()
	TObjectPtr<const UScriptStruct> Struct;

	/** The node this pin connects to */
	UPROPERTY()
	TObjectPtr<const UDataLinkNode> LinkedNode;

	/** the pin index at which this pin connects to the linked node (to find linked pin) */
	UPROPERTY()
	int32 LinkedIndex = INDEX_NONE;
};
