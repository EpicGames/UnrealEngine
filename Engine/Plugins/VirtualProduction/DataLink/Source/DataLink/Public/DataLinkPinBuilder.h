// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DataLinkPin.h"

/** Short-lived parameter building for a single pin */
struct FDataLinkPinParameters
{
	DATALINK_API explicit FDataLinkPinParameters(FDataLinkPin& InPin);

	/** 
	 * Optionally sets the display text of the pin
	 * If not set, it will use the pin's unique name instead.
	 */
	DATALINK_API FDataLinkPinParameters& SetDisplayName(FText&& InPinDisplayName);

	template<typename InStruct UE_REQUIRES(TModels_V<CStaticStructProvider, InStruct>)>
	FDataLinkPinParameters& SetStruct()
	{
		return this->SetStruct(InStruct::StaticStruct());
	}

	/**
	 * Sets the struct of the pin.
	 * Can be left null for cases like output pins where the struct might not be known
	 */
	DATALINK_API FDataLinkPinParameters& SetStruct(const UScriptStruct* InPinStruct);

private:
	FDataLinkPin& Pin;
};

/** Short-lived builder for pin arrays limiting to only add pins with unique names */
struct FDataLinkPinBuilder
{
	explicit FDataLinkPinBuilder(TArray<FDataLinkPin>& OutPins);

	/**
	 * Possibly re-allocates the array to ensure its capacity can hold the number of items to add without further re-allocations
	 * @param InNumToAdd the number of new elements to add
	 */
	DATALINK_API void AddCapacity(int32 InNumToAdd);

	/**
	 * Adds a new pin to the array with a given unique name
	 * @param InPinName the unique name of the pin
	 * @return parameter object to further customize the pin
	 */
	DATALINK_API FDataLinkPinParameters Add(FName InPinName);

private:
	TArray<FDataLinkPin>& Pins;
};
