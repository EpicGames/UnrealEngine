// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"

class UDataLinkNode;
struct FDataLinkPin;

/** Short-lived data containing a pin reference with the Node that owns it. */
struct FDataLinkPinReference
{
	explicit FDataLinkPinReference(const UDataLinkNode* InOwningNode, const FDataLinkPin* InPin)
		: OwningNode(InOwningNode)
		, Pin(InPin)
	{
	}

	TObjectPtr<const UDataLinkNode> OwningNode;

	const FDataLinkPin* Pin = nullptr;
};
