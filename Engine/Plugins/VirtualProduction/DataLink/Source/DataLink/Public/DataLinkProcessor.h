// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructView.h"
#include "UObject/Object.h"
#include "DataLinkProcessor.generated.h"

class FDataLinkExecutor;
enum class EDataLinkExecutionResult : uint8;

/** Processors execute logic on the output data of a data link graph execution */
UCLASS(MinimalAPI, Abstract, EditInlineNew)
class UDataLinkProcessor : public UObject
{
	GENERATED_BODY()

public:
	/** Called at the start of the data link graph execution, prior to running any data link node */
	void Initialize(const FDataLinkExecutor& InExecutor);

	/** Called when the data link graph execution has output data */
	void ProcessOutput(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView);

	/** Called at the end of the data link graph execution */
	void Finalize(const FDataLinkExecutor& InExecutor, EDataLinkExecutionResult InExecutionResult);

protected:
	virtual void OnInitialize(const FDataLinkExecutor& InExecutor)
	{
	}

	virtual void OnProcessOutput(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView)
	{
	}

	virtual void OnFinalize(const FDataLinkExecutor& InExecutor, EDataLinkExecutionResult InExecutionResult)
	{
	}
};
