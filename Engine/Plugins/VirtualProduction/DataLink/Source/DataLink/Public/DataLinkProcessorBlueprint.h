// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkProcessor.h"
#include "DataLinkProcessorBlueprint.generated.h"

UCLASS(MinimalAPI, Abstract, Blueprintable)
class UDataLinkProcessorBlueprint : public UDataLinkProcessor
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category="Data Link")
	UObject* GetContextObject() const;

	/** Called at the start of the data link graph execution, prior to running any data link node */
	UFUNCTION(BlueprintImplementableEvent)
	void Initialize();

	/** Called when the data link graph execution has output data */
	UFUNCTION(BlueprintImplementableEvent)
	void ProcessOutput(const FInstancedStruct& InOutputData);

	/** Called at the end of the data link graph execution */
	UFUNCTION(BlueprintImplementableEvent)
	void Finalize(EDataLinkExecutionResult InExecutionResult);

protected:
	//~ Begin UDataLinkProcessor
	virtual void OnInitialize(const FDataLinkExecutor& InExecutor);
	virtual void OnProcessOutput(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView) override;
	virtual void OnFinalize(const FDataLinkExecutor& InExecutor, EDataLinkExecutionResult InExecutionResult) override;
	//~ End UDataLinkProcessor

	//~ Begin UObject
	virtual UWorld* GetWorld() const override;
	//~ End UObject

	/**
	 * Cached output data to avoid memory allocation if the output keeps being the same struct. Only reallocs if struct mismatches.
	 * Unlike FConstStructView FInstancedStruct is supported in Blueprints.
	 */
	UPROPERTY()
	FInstancedStruct OutputData;

	/** Object responsible for the execution */
	TWeakPtr<const FDataLinkExecutor> ExecutorWeak;
};
