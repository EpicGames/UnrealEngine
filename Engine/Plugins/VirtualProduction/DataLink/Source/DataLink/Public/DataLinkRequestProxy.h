// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "UObject/Object.h"
#include "DataLinkRequestProxy.generated.h"

class IDataLinkSinkProvider;
class FDataLinkExecutor;
enum class EDataLinkExecutionResult : uint8;
struct FDataLinkInstance;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnDataLinkOutputDataProxy, const FInstancedStruct&, OutputData, EDataLinkExecutionResult, ExecutionResult);

UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "Data Link Request Proxy deprecated. Use Data Link Executor Object instead") UDataLinkRequestProxy : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintAssignable)
	FOnDataLinkOutputDataProxy OnOutputData;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Data Link Request Proxy deprecated. Use Data Link Executor Object instead")
	UFUNCTION(BlueprintCallable, Category="Data Link", meta=(BlueprintInternalUseOnly))
	static DATALINK_API UDataLinkRequestProxy* CreateRequestProxy(FDataLinkInstance InDataLinkInstance
		, UObject* InExecutionContext
		, TScriptInterface<IDataLinkSinkProvider> InDataLinkSinkProvider);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	DATALINK_API void ProcessRequest(FDataLinkInstance&& InDataLinkInstance
		, UObject* InExecutionContext
		, TScriptInterface<IDataLinkSinkProvider> InDataLinkSinkProvider);

private:
	void OnOutputDataReceived(const FDataLinkExecutor& InExecutor, FConstStructView InOutputData);

	TSharedPtr<FDataLinkExecutor> DataLinkExecutor;
};
