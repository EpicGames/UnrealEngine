// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkInputDataViewer.h"
#include "Misc/Optional.h"
#include "StructUtils/StructView.h"
#include "UObject/ObjectKey.h"

class FDataLinkInputDataViewer;
class UDataLinkNode;
struct FDataLinkNodeInstance;

/** Struct used as the Key to access the data for a given node and input data */
struct FDataLinkSinkKey
{
	FDataLinkSinkKey() = default;

	explicit FDataLinkSinkKey(TSubclassOf<UDataLinkNode> InNodeClass, const FDataLinkInputDataViewer& InInputDataViewer);

	bool operator==(const FDataLinkSinkKey& InOther) const;

	friend uint64 GetTypeHash(const FDataLinkSinkKey& InKey);

private:
	/** Class of the node that this data will be used in */
	FObjectKey NodeClass;

	/** Views of the input data. */
	TArray<FDataLinkInputDataEntry, TInlineAllocator<2>> InputDataEntries;

	/** Hash computed in ctor */
	uint64 CachedHash = 0;
};

/** Sink that caches node data to be reusable by nodes with the same logic (i.e. "class") and input data */
struct FDataLinkSink
{
	/**
	 * Returns the output data view that matches the node class and the settings within the input data view.
	 * @return a data view to the output data
	 */
	DATALINK_API FInstancedStruct& FindOrAddCachedData(const FDataLinkNodeInstance& InNodeInstance, TOptional<const UScriptStruct*> InDesiredStruct = {});

	template<typename T>
	T& FindOrAddCachedData(const FDataLinkNodeInstance& InNodeInstance)
	{
		FInstancedStruct& InstancedStruct = this->FindOrAddCachedData(InNodeInstance, T::StaticStruct());
		return InstancedStruct.GetMutable<T>();
	}

	void AddStructReferencedObjects(FReferenceCollector& InCollector);

private:
	/**
	 * Map of the Node & Input Data to the Cached Data
	 * This stores output data for Input data regardless if it's hashable or not.
	 * For input data without a type hash implementation, it should return a type hash of 0, generating collisions that then should get the correct element via comparison.
	 */
	TMap<FDataLinkSinkKey, FInstancedStruct> CachedDataMap;
};
