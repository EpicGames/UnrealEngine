// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDataLinkSinkProvider.h"
#include "UObject/Object.h"
#include "DataLinkSinkObject.generated.h"

struct FDataLinkSink;

UCLASS(MinimalAPI, BlueprintType)
class UDataLinkSinkObject : public UObject, public IDataLinkSinkProvider
{
	GENERATED_BODY()

public:
	UDataLinkSinkObject();

	UFUNCTION(BlueprintCallable, Category="Data Link")
	void ResetSink();

	//~ Begin IDataLinkSinkProvider
	virtual TSharedPtr<FDataLinkSink> GetSink() const override;
	virtual const UDataLinkSinkObject* GetSinkObject_Implementation() const override;
	//~ End IDataLinkSinkProvider

private:
	TSharedPtr<FDataLinkSink> Sink;
};
