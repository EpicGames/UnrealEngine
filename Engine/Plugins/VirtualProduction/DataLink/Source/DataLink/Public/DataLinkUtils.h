// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointerFwd.h"

class FProperty;
class IDataLinkSinkProvider;
class UClass;
class UDataLinkGraph;
class UObject;
class UStruct;
struct FConstStructView;
struct FDataLinkInputData;
struct FDataLinkSink;
struct FStructView;
template <typename T> class TScriptInterface;

#define UE_API DATALINK_API

namespace UE::DataLink
{
	/**
	 * Copies a given source view into dest view if the script structs match
	 * @param InDestView the struct view to copy to
	 * @param InSourceView the struct view to copy from
	 * @return true if the struct views were compatible and the copy was performed
	 */
	UE_API bool CopyDataView(FStructView InDestView, FConstStructView InSourceView);

	/**
	 * Replaces a given Object with a new object with the same name but different class
	 * @param InOutObject object to replace. Can come in as null
	 * @param InOuter outer of the object. Must be valid.
	 * @param InClass new class of the new object that will replace the older one
	 * @return true if the operation took place, false otherwise
	 */
	UE_API bool ReplaceObject(UObject*& InOutObject, UObject* InOuter, UClass* InClass);

	/**
	 * Attempts to get the underlying Sink from the given Sink Provider
	 * @param InSinkProvider the sink provider interface
	 * @return the sink if found
	 */
	UE_API TSharedPtr<FDataLinkSink> TryGetSink(TScriptInterface<IDataLinkSinkProvider> InSinkProvider);

	struct FConstPropertyView
	{
		const FProperty* Property = nullptr;
		const uint8* Memory = nullptr;
	};
	/**
	 * Sets the input data to match the current graph's inputs
	 * @param InBaseStructView the base struct to start looking for the property path
	 * @param InPropertyPath the property path to look for
	 * @param OutError optional error message if property returns null
	 * @return the resolved property view (property and memory ptr)
	 */
	UE_API FConstPropertyView ResolveConstPropertyView(FConstStructView InBaseStructView, const FString& InPropertyPath, FString* OutError = nullptr);

	struct FPropertyView
	{
		FProperty* Property = nullptr;
		uint8* Memory = nullptr;
	};
	/**
	 * Sets the input data to match the current graph's inputs
	 * @param InBaseStructView the base struct to start looking for the property path
	 * @param InPropertyPath the property path to look for
	 * @param OutError optional error message if property returns null
	 * @return the resolved property view (property and memory ptr)
	 */
	UE_API FPropertyView ResolvePropertyView(FStructView InBaseStructView, const FString& InPropertyPath, FString* OutError = nullptr);

	/**
	 * Sets the input data to match the current graph's inputs
	 * @param InGraph the data link graph to get the input pins to update from
	 * @param OutInputData the input data to update
	 */
	UE_API void SetInputData(UDataLinkGraph* InGraph, TArray<FDataLinkInputData>& OutInputData);

	/** Converts the given struct data view to a debug string */
	UE_API FString StructViewToDebugString(FConstStructView InDataView);
}

#undef UE_API
