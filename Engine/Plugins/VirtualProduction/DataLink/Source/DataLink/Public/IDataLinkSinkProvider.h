// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IDataLinkSinkProvider.generated.h"

class UDataLinkSinkObject;
struct FDataLinkSink;

UINTERFACE(MinimalAPI)
class UDataLinkSinkProvider : public UInterface
{
	GENERATED_BODY()
};

class IDataLinkSinkProvider
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, Category="Data Link")
	DATALINK_API const UDataLinkSinkObject* GetSinkObject() const;

	virtual TSharedPtr<FDataLinkSink> GetSink() const
	{
		return nullptr;
	}
};
