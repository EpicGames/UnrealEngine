// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "StructUtils/InstancedStruct.h"
#include "DataLinkConstant.generated.h"

/** Constants are a no-input node that provide a struct instance that does not change in execution time */
UCLASS(MinimalAPI, DisplayName="Constant", Category="Core")
class UDataLinkConstant : public UDataLinkNode
{
	GENERATED_BODY()

public:
	DATALINK_API void SetStruct(const UScriptStruct* InStructType);

protected:
	//~ Begin UDataLinkNode
#if WITH_EDITOR
	DATALINK_API virtual void OnFixupNode() override;
	DATALINK_API virtual void OnBuildMetadata(FDataLinkNodeMetadata& Metadata) const override;
#endif
	DATALINK_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
	DATALINK_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
	//~ End UDataLinkNode

private:
	UPROPERTY(EditAnywhere, Category="Data Link", meta=(InvalidatesNode))
	FText DisplayName; 

	UPROPERTY(EditAnywhere, Category="Data Link", meta=(InvalidatesNode))
	FInstancedStruct Instance;
};
