// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "DataLinkScriptNode.generated.h"

struct FInstancedStruct;

USTRUCT()
struct FDataLinkScriptPin
{
	GENERATED_BODY()

	/** Unique name for the Pin */
	UPROPERTY(EditAnywhere, Category = "DataLink")
	FName Name;

	/** Struct type of the Pin */
	UPROPERTY(EditAnywhere, Category = "DataLink")
	TObjectPtr<const UScriptStruct> Struct;
};

/**
 * Script Nodes are a Blueprint Implementation of a Data Link Node.
 * It does not inherit from UDataLinkNode, as the wrapper does this and forwards the logic execution here.
 * This is done to allow for blueprints mutable nature (e.g. allowing users to set variables)
 * @see UDataLinkScriptNodeWrapper
 */
UCLASS(Abstract, MinimalAPI, Blueprintable)
class UDataLinkScriptNode : public UObject
{
	GENERATED_BODY()

	friend class UDataLinkScriptNodeWrapper;

public:
	void Execute(const UDataLinkNode* InNode, FDataLinkExecutor& InExecutor);

	void Stop();

	UFUNCTION(BlueprintImplementableEvent)
	void OnExecute();

	UFUNCTION(BlueprintImplementableEvent)
	void OnStop();

	/**
	 * Called to move to the next node to execute (or finish if last node) providing the output data as an instanced struct
	 * Set Persist Execution to true so that the graph does not finish, but keeps pushing data.
	 */
	UFUNCTION(BlueprintCallable, Category="Data Link", DisplayName="Succeed")
	bool Succeed(const FInstancedStruct& OutputData, bool bPersistExecution);

	/** Called to move to the next node to execute (or finish if last node) and providing the output data as a wildcard */
	UFUNCTION(BlueprintCallable, CustomThunk, Category="Data Link", DisplayName="Succeed (Wildcard)", meta=(CustomStructureParam="OutputData"))
	bool SucceedWildcard(int32 OutputData, bool bPersistExecution);

	/** Called to fail execution */
	UFUNCTION(BlueprintCallable, Category="Data Link")
	void Fail();

	/** Retrieves the input data as an Instanced Struct */
	UFUNCTION(BlueprintCallable, Category="Data Link", DisplayName="Get Input Data")
	bool GetInputData(FInstancedStruct& InputData, FName InputName) const;

	/** Retrieves the input data as a Wildcard */
	UFUNCTION(BlueprintCallable, CustomThunk, Category="Data Link", DisplayName="Get Input Data (Wildcard)", meta=(CustomStructureParam="InputData"))
	bool GetInputDataWildcard(int32& InputData, FName InputName) const;

protected:
	//~ Begin UObject
	virtual UWorld* GetWorld() const override;
	//~ End UObject

private:
	UObject* GetContextObject() const;

	DECLARE_FUNCTION(execSucceedWildcard);
	DECLARE_FUNCTION(execGetInputDataWildcard);

	UPROPERTY(EditDefaultsOnly, Category="Data Link")
	TArray<FDataLinkScriptPin> InputPins;

	UPROPERTY(EditDefaultsOnly, Category="Data Link")
	FDataLinkScriptPin OutputPin;

	UPROPERTY()
	TObjectPtr<const UDataLinkNode> Node;

	TWeakPtr<FDataLinkExecutor> ExecutorWeak;
};
