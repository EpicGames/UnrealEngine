// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "Templates/SubclassOf.h"
#include "DataLinkScriptNodeWrapper.generated.h"

#define UE_API DATALINK_API

class UDataLinkScriptNode;

/** Wrapper struct for the Node Instance containing the actual blueprint implementation of the data link logic */
USTRUCT()
struct FDataLinkScriptNodeInstance
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UDataLinkScriptNode> Node;
};

/**
 * Wrapper to the actual blueprint implementation of the node.
 * Data Link Nodes are by-design immutable in execution, and only modify input/output/instance data provided by the executor.
 * Data Link Script Nodes are mutable (like typical blueprints) so their actual implementation (UDataLinkScriptNode) is instanced in every execution
 * @see UDataLinkScriptNode
 */
UCLASS(MinimalAPI, DisplayName="Blueprint Node", meta=(Hidden))
class UDataLinkScriptNodeWrapper : public UDataLinkNode
{
	GENERATED_BODY()

public:
	UE_API UDataLinkScriptNodeWrapper();

	static FName GetNodeClassPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UDataLinkScriptNodeWrapper, NodeClass);
	}

	UE_API void SetNodeClass(TSubclassOf<UDataLinkScriptNode> InNodeClass);

protected:
	//~ Begin UDataLinkNode
#if WITH_EDITOR
	UE_API virtual void OnBuildMetadata(FDataLinkNodeMetadata& Metadata) const override;
#endif
	UE_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
	UE_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
	UE_API virtual void OnStop(const FDataLinkExecutor& InExecutor) const override;
	//~ End UDataLinkNode

private:
	UDataLinkScriptNode* GetDefaultNode() const;

	/**
	 * The node class to instantiate.
	 * The selected node class determines the pins of this node.
	 */
	UPROPERTY(meta=(InvalidatesNode))
	TSubclassOf<UDataLinkScriptNode> NodeClass;

	/** Instance of Node Class serving as template for the execution instances */
	UPROPERTY(VisibleAnywhere, Instanced, Category="Data Link")
	TObjectPtr<UDataLinkScriptNode> TemplateNode;
};

#undef UE_API
