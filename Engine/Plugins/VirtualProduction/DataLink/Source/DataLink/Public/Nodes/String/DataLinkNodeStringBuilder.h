// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "DataLinkStringBuilder.h"
#include "DataLinkNodeStringBuilder.generated.h"

/** Builder Node to help form a more complex string with parameters */
UCLASS(MinimalAPI, DisplayName="String Builder", Category="Core")
class UDataLinkNodeStringBuilder : public UDataLinkNode
{
	GENERATED_BODY()

protected:
	//~ Begin UDataLinkNode
	DATALINK_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
	DATALINK_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
	//~ End UDataLinkNode

	//~ Begin UObject
#if WITH_EDITOR
	DATALINK_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

private:
	/** The Segments to build, where a token can be defined as {Token Name} in its own isolated entry */
	UPROPERTY(EditAnywhere, Category="Data Link", meta=(InvalidatesNode))
	TArray<FString> Segments;

	/** The tokens found within the Segment Array */
	UPROPERTY()
	TArray<FDataLinkStringBuilderToken> Tokens;
};
