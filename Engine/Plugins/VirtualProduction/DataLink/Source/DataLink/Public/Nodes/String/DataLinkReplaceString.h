// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "DataLinkReplaceString.generated.h"

#define UE_API DATALINK_API

namespace UE::DataLink
{
	const FLazyName InputReplaceSettings(TEXT("InputReplaceSettings"));
}

USTRUCT(BlueprintType)
struct FDataLinkReplaceStringEntry
{
	GENERATED_BODY()

	/** The regex pattern to match */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Replace")
	FString Pattern;

	/** The replacement string, can use capture groups with $ (e.g. $1)*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Replace")
	FString Replacement;
};

USTRUCT(BlueprintType)
struct FDataLinkReplaceStringSettings
{
	GENERATED_BODY()

	/** Entries to run in order */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Regex")
	TArray<FDataLinkReplaceStringEntry> ReplaceEntries;
};

/** Replaces a string multiple times with simple or regex patterns */
UCLASS(MinimalAPI, DisplayName="Replace String", Category="Core")
class UDataLinkReplaceString : public UDataLinkNode
{
	GENERATED_BODY()

protected:
	//~ Begin UDataLinkNode
	UE_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
	UE_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
	//~ End UDataLinkNode
};

#undef UE_API
