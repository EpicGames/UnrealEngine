// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "Templates/FunctionFwd.h"
#include "DataLinkStringBuilder.generated.h"

class FDataLinkInputDataViewer;
class FString;
struct FConstStructView;
struct FDataLinkPinBuilder;

USTRUCT()
struct FDataLinkStringBuilderToken
{
	GENERATED_BODY()

	/** Name of the token. Used to build the Input Pin */
	UPROPERTY()
	FName Name;

	/** The indices this token is located in the Segment array */
	UPROPERTY()
	TArray<int32> Indices;
};

/**
 * Logic for String building for re-usability between different nodes without needing to inherit from each other
 * @see UDataLinkNodeStringBuilder
 */
struct FDataLinkStringBuilder
{
	explicit FDataLinkStringBuilder(TConstArrayView<FString> InSegments, TConstArrayView<FDataLinkStringBuilderToken> InTokens)
		: Segments(InSegments)
		, Tokens(InTokens)
	{
	}

	DATALINK_API bool BuildString(const FDataLinkInputDataViewer& InTokenValues, FString& InOutResult) const;

	DATALINK_API void BuildInputPins(FDataLinkPinBuilder& Inputs) const;

	DATALINK_API static void GatherTokens(TConstArrayView<FString> InSegments, TArray<FDataLinkStringBuilderToken>& OutTokens);

private:
	void ForEachResolvedSegment(const FDataLinkInputDataViewer& InTokenValues, TFunctionRef<void(FStringView InResolvedSegment)> InFunction) const;

	TConstArrayView<FString> Segments;

	TConstArrayView<FDataLinkStringBuilderToken> Tokens;
};
