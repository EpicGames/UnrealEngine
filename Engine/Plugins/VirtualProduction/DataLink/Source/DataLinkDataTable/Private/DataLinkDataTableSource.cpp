// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkDataTableSource.h"
#include "DataLinkExecutor.h"
#include "DataLinkNames.h"
#include "DataLinkNodeInstance.h"
#include "DataLinkPinBuilder.h"
#include "DataLinkUtils.h"
#include "StructUtils/StructView.h"

void UDataLinkDataTableSource::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	Super::OnBuildPins(Inputs, Outputs);

	Inputs.Add(UE::DataLinkDataTable::InputRow)
		.SetStruct<FDataTableRowHandle>();

	// Output pin does not have a known struct without knowing what the data table to use is
	Outputs.Add(UE::DataLink::OutputDefault);
}

EDataLinkExecutionReply UDataLinkDataTableSource::OnExecute(FDataLinkExecutor& InExecutor) const
{
	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	const FDataLinkInputDataViewer& InputDataViewer = NodeInstance.GetInputDataViewer();
	const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

	const FDataTableRowHandle& RowHandle = InputDataViewer.Get<FDataTableRowHandle>(UE::DataLinkDataTable::InputRow);
	if (!RowHandle.DataTable)
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	const uint8* RowMemory = RowHandle.DataTable->FindRowUnchecked(RowHandle.RowName);
	if (!RowMemory)
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	// Row memory will return null if the Row struct is null. 
	check(RowHandle.DataTable->RowStruct != nullptr);

	const FStructView OutputDataView = OutputDataViewer.Find(UE::DataLink::OutputDefault, RowHandle.DataTable->RowStruct);
	if (!UE::DataLink::CopyDataView(OutputDataView, FConstStructView(RowHandle.DataTable->RowStruct, RowMemory)))
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	InExecutor.Next(this);
	return EDataLinkExecutionReply::Handled;
}
