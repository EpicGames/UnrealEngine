// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "Engine/DataTable.h"
#include "DataLinkDataTableSource.generated.h"

namespace UE::DataLinkDataTable
{
    const FLazyName InputRow(TEXT("InputRow"));
}

UCLASS(MinimalAPI, Category="Data Table", DisplayName="Data Table")
class UDataLinkDataTableSource : public UDataLinkNode
{
    GENERATED_BODY()

protected:
    //~ Begin UDataLinkNode
    DATALINKDATATABLE_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
    DATALINKDATATABLE_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
    //~ End UDataLinkNode
};
