// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataLinkEdGraph : ModuleRules
{
    public DataLinkEdGraph(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "DataLink",
                "Engine",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "BlueprintGraph",
            }
        );

        if (Target.Type == TargetRules.TargetType.Editor)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "UnrealEd",
                    "KismetCompiler",
                }
            );
        }
    }
}
