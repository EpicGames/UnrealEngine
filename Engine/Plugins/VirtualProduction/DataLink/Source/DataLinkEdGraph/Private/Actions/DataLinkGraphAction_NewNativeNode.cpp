// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphAction_NewNativeNode.h"
#include "DataLinkNode.h"
#include "Nodes/DataLinkConstant.h"

FDataLinkGraphAction_NewNativeNode::FDataLinkGraphAction_NewNativeNode(TSubclassOf<UDataLinkNode> InNodeClass, int32 InGrouping)
	: NodeClass(InNodeClass)
{
	Grouping = InGrouping;

	UpdateSearchData(InNodeClass->GetDisplayNameText()
		, InNodeClass->GetToolTipText()
		, FText::GetEmpty()
		, FText::GetEmpty());
}

TSubclassOf<UDataLinkNode> FDataLinkGraphAction_NewNativeNode::GetNodeClass() const
{
	return NodeClass;
}

void FDataLinkGraphAction_NewNativeNode::ConfigureNode(const FConfigContext& InContext) const
{
	if (!InContext.SourcePin || !InContext.TemplateNode)
	{
		return;
	}

	if (UDataLinkConstant* ConstantNode = Cast<UDataLinkConstant>(InContext.TemplateNode))
	{
		const UScriptStruct* Struct = Cast<UScriptStruct>(InContext.SourcePin->PinType.PinSubCategoryObject);
		ConstantNode->SetStruct(Struct);
	}
}
