// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkGraphAction_NewNode.h"
#include "Templates/SubclassOf.h"
#include "DataLinkGraphAction_NewNativeNode.generated.h"

class UDataLinkNode;

USTRUCT()
struct FDataLinkGraphAction_NewNativeNode : public FDataLinkGraphAction_NewNode
{
	GENERATED_BODY()

	FDataLinkGraphAction_NewNativeNode() = default;

	explicit FDataLinkGraphAction_NewNativeNode(TSubclassOf<UDataLinkNode> InNodeClass, int32 InGrouping);

protected:
	//~ Begin FDataLinkGraphAction_NewNode
	virtual TSubclassOf<UDataLinkNode> GetNodeClass() const override;
	virtual void ConfigureNode(const FConfigContext& InContext) const override;
	//~ End FDataLinkGraphAction_NewNode

private:
	TSubclassOf<UDataLinkNode> NodeClass;
};
