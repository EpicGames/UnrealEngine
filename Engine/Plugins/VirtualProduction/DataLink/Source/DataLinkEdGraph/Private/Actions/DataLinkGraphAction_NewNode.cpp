// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphAction_NewNode.h"
#include "DataLinkNode.h"
#include "EdGraph/EdGraph.h"
#include "Nodes/DataLinkEdNode.h"
#include "ScopedTransaction.h"
#include "Settings/EditorStyleSettings.h"

#define LOCTEXT_NAMESPACE "DataLinkGraphAction_NewNode"

UEdGraphNode* FDataLinkGraphAction_NewNode::PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InSourcePin, const FVector2f& InLocation, bool bInSelectNewNode)
{
	if (!InParentGraph)
	{
		return nullptr;
	}

	TSubclassOf<UDataLinkNode> TemplateNodeClass = GetNodeClass();
	if (!TemplateNodeClass)
	{
		return nullptr;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddNode", "Add Node"));

	InParentGraph->Modify();
	if (InSourcePin)
	{
		InSourcePin->Modify();
	}

	UDataLinkEdNode* EdNode = NewObject<UDataLinkEdNode>(InParentGraph);
	check(EdNode);

	EdNode->SetFlags(RF_Transactional);

	InParentGraph->AddNode(EdNode, /*bUserAction*/true, bInSelectNewNode);

	EdNode->CreateNewGuid();
	EdNode->PostPlacedNewNode();
	EdNode->AllocateDefaultPins();
	EdNode->SetTemplateNodeClass(TemplateNodeClass, /*bReconstructNode*/false);

	{
		FConfigContext ConfigContext;
		ConfigContext.SourcePin = InSourcePin;
		ConfigContext.TemplateNode = EdNode->GetTemplateNode();
		ConfigureNode(ConfigContext);
	}

	EdNode->ReconstructNode();
	EdNode->AutowireNewNode(InSourcePin);

	EdNode->NodePosX = static_cast<int32>(InLocation.X);
	EdNode->NodePosY = static_cast<int32>(InLocation.Y);

	EdNode->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);

	return EdNode;
}

#undef LOCTEXT_NAMESPACE
