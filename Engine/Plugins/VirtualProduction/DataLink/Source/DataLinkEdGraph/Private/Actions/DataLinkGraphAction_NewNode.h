// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "Templates/SubclassOf.h"
#include "DataLinkGraphAction_NewNode.generated.h"

class UDataLinkNode;

USTRUCT()
struct FDataLinkGraphAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	FDataLinkGraphAction_NewNode() = default;

protected:
	virtual TSubclassOf<UDataLinkNode> GetNodeClass() const
	{
		return nullptr;
	}

	struct FConfigContext
	{
		UDataLinkNode* TemplateNode;
		UEdGraphPin* SourcePin;
	};
	virtual void ConfigureNode(const FConfigContext& InContext) const
	{
	}

	//~ Begin FEdGraphSchemaAction
	virtual UEdGraphNode* PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InSourcePin, const FVector2f& InLocation, bool bInSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction
};
