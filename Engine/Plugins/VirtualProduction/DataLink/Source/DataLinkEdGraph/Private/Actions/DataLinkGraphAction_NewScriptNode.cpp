// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphAction_NewScriptNode.h"
#include "Blueprint/BlueprintSupport.h"
#include "Engine/Blueprint.h"
#include "Nodes/DataLinkEdNode.h"
#include "Nodes/Script/DataLinkScriptNode.h"
#include "Nodes/Script/DataLinkScriptNodeWrapper.h"
#include "UObject/Class.h"

#define LOCTEXT_NAMESPACE "DataLinkGraphAction_NewScriptNode"

FDataLinkGraphAction_NewScriptNode::FDataLinkGraphAction_NewScriptNode(const FAssetData& InNodeAsset, int32 InGrouping)
	: NodeAsset(InNodeAsset)
{
	Grouping = InGrouping;

	auto GetTag = [&InNodeAsset](FName InTagKey, const FText& InDefaultValue = FText::GetEmpty())
		{
			FString TagValue;
			if (InNodeAsset.GetTagValue(InTagKey, TagValue))
			{
				return FText::FromString(TagValue);
			}
			return InDefaultValue;
		};

	const FText AssetNameText = FText::FromName(InNodeAsset.AssetName);
	UpdateSearchData(GetTag(FBlueprintTags::BlueprintDisplayName, AssetNameText)
		, GetTag(FBlueprintTags::BlueprintDescription)
		, GetTag(FBlueprintTags::BlueprintCategory, LOCTEXT("DefaultBlueprintCategory", "Script Nodes"))
		, FText::GetEmpty());
}

TSubclassOf<UDataLinkNode> FDataLinkGraphAction_NewScriptNode::GetNodeClass() const
{
	return UDataLinkScriptNodeWrapper::StaticClass();
}

void FDataLinkGraphAction_NewScriptNode::ConfigureNode(const FConfigContext& InContext) const
{
	if (UDataLinkScriptNodeWrapper* ScriptNodeWrapper = Cast<UDataLinkScriptNodeWrapper>(InContext.TemplateNode))
	{
		ScriptNodeWrapper->SetNodeClass(GetScriptNodeClass());
	}
}

TSubclassOf<UDataLinkScriptNode> FDataLinkGraphAction_NewScriptNode::GetScriptNodeClass() const
{
	UObject* NodeObject = NodeAsset.GetAsset();
	if (!NodeObject)
	{
		return nullptr;
	}

	if (TSubclassOf<UDataLinkScriptNode> NodeClass = Cast<UClass>(NodeObject))
	{
		return NodeClass;
	}

	if (UBlueprint* Blueprint = Cast<UBlueprint>(NodeObject))
	{
		return Cast<UClass>(Blueprint->GeneratedClass);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
