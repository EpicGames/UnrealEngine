// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkGraphAction_NewNode.h"
#include "Nodes/Script/DataLinkScriptNode.h"
#include "DataLinkGraphAction_NewScriptNode.generated.h"

USTRUCT()
struct FDataLinkGraphAction_NewScriptNode : public FDataLinkGraphAction_NewNode
{
	GENERATED_BODY()

	FDataLinkGraphAction_NewScriptNode() = default;

	explicit FDataLinkGraphAction_NewScriptNode(const FAssetData& InNodeAsset, int32 InGrouping);

protected:
	//~ Begin FDataLinkGraphAction_NewNode
	virtual TSubclassOf<UDataLinkNode> GetNodeClass() const override;
	virtual void ConfigureNode(const FConfigContext& InContext) const override;
	//~ End FDataLinkGraphAction_NewNode

private:
	TSubclassOf<UDataLinkScriptNode> GetScriptNodeClass() const;

	FAssetData NodeAsset;
};
