// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_AsyncDataLinkRequest.h"
#include "DataLinkRequestProxy.h"

UK2Node_AsyncDataLinkRequest::UK2Node_AsyncDataLinkRequest()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UDataLinkRequestProxy, CreateRequestProxy);
	ProxyFactoryClass = UDataLinkRequestProxy::StaticClass();
	ProxyClass = UDataLinkRequestProxy::StaticClass();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UEdGraphPin* UK2Node_AsyncDataLinkRequest::GetDataLinkInstancePin() const
{
	return FindPinChecked(TEXT("InDataLinkInstance"), EGPD_Input);
}

UEdGraphPin* UK2Node_AsyncDataLinkRequest::GetExecutionContextPin() const
{
	return FindPinChecked(TEXT("InExecutionContext"), EGPD_Input);
}

UEdGraphPin* UK2Node_AsyncDataLinkRequest::GetDataLinkSinkProviderPin() const
{
	return FindPinChecked(TEXT("InDataLinkSinkProvider"), EGPD_Input);
}

UEdGraphPin* UK2Node_AsyncDataLinkRequest::GetOutputDataPin() const
{
	return FindPinChecked(TEXT("OutputData"), EGPD_Output);
}

UEdGraphPin* UK2Node_AsyncDataLinkRequest::GetExecutionResultPin() const
{
	return FindPinChecked(TEXT("ExecutionResult"), EGPD_Output);
}

UEdGraphPin* UK2Node_AsyncDataLinkRequest::GetOnOutputDataPin() const
{
	return FindPinChecked(TEXT("OnOutputData"), EGPD_Output);
}

void UK2Node_AsyncDataLinkRequest::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	// Hide this node from menu actions
}
