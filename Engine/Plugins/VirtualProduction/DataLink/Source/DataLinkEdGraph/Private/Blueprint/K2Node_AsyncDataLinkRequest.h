// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_BaseAsyncTask.h"
#include "K2Node_AsyncDataLinkRequest.generated.h"

UCLASS()
class UK2Node_AsyncDataLinkRequest : public UK2Node_BaseAsyncTask
{
	GENERATED_BODY()

public:
	UK2Node_AsyncDataLinkRequest();

	UEdGraphPin* GetDataLinkInstancePin() const;
	UEdGraphPin* GetExecutionContextPin() const;
	UEdGraphPin* GetDataLinkSinkProviderPin() const;
	UEdGraphPin* GetOutputDataPin() const;
	UEdGraphPin* GetExecutionResultPin() const;
	UEdGraphPin* GetOnOutputDataPin() const;

protected:
	//~ Begin UK2Node
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const override;
	//~ End UK2Node
};
