// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/K2Node_DataLinkRequest.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "DataLinkEnums.h"
#include "DataLinkGraph.h"
#include "DataLinkInstance.h"
#include "IDataLinkSinkProvider.h"
#include "K2Node_AsyncDataLinkRequest.h"
#include "K2Node_SwitchEnum.h"
#include "KismetCompiler.h"
#include "Nodes/DataLinkEdNode.h"
#include "StructUtils/InstancedStruct.h"

#define LOCTEXT_NAMESPACE "K2Node_DataLinkRequest"

const FLazyName UK2Node_DataLinkRequest::PN_DataLinkInstance = TEXT("DataLinkInstance");
const FLazyName UK2Node_DataLinkRequest::PN_ExecutionContext = TEXT("ExecutionContext");
const FLazyName UK2Node_DataLinkRequest::PN_DataLinkSinkProvider = TEXT("DataLinkSinkProvider");
const FLazyName UK2Node_DataLinkRequest::PN_Success = TEXT("Success");
const FLazyName UK2Node_DataLinkRequest::PN_Failure = TEXT("Failure");

void UK2Node_DataLinkRequest::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	// Exec Pin
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	// Then Pin
	UEdGraphPin* const ThenPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	ThenPin->PinToolTip = TEXT("For execution that needs to happen immediately without waiting for a response.");

	// Success Pin
	UEdGraphPin* const SuccessPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, PN_Success);
	SuccessPin->PinFriendlyName = LOCTEXT("SuccessPinLabel", "Output Data");
	SuccessPin->PinToolTip = TEXT("Called when the execution received and processed data successfully");

	// Create Output Pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FInstancedStruct::StaticStruct(), UDataLinkEdNode::PN_Output);

	// Failure pin
	UEdGraphPin* const FailurePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, PN_Failure);
	FailurePin->PinFriendlyName = LOCTEXT("FailurePinLabel", "Data Link Failed");
	FailurePin->PinToolTip = TEXT("Called when the execution failed and no data will be received");

	// Create Data Link Instance Pin
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FDataLinkInstance::StaticStruct(), PN_DataLinkInstance);

	// Create Execution Context Pin
	if (UEdGraphPin* ExecutionContextPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UObject::StaticClass(), PN_ExecutionContext))
	{
		// Prevent user from editing default value (i.e. hiding the object dropdown as option).
		ExecutionContextPin->bDefaultValueIsIgnored = true;
		ExecutionContextPin->bAdvancedView = true;
	}

	// Create Data Link Sink Provider Pin
	if (UEdGraphPin* SinkProviderPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Interface, UDataLinkSinkProvider::StaticClass(), PN_DataLinkSinkProvider))
	{
		// Prevent user from editing default value (i.e. hiding the object dropdown as option).
		SinkProviderPin->bDefaultValueIsIgnored = true;
		SinkProviderPin->bAdvancedView = true;
	}

	if (AdvancedPinDisplay == ENodeAdvancedPins::NoPins)
	{
		AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
	}
}

bool UK2Node_DataLinkRequest::IsCompatibleWithGraph(const UEdGraph* InTargetGraph) const
{
	const UEdGraphSchema* Schema = InTargetGraph->GetSchema();
	if (!Schema)
	{
		return false;
	}

	const EGraphType GraphType = Schema->GetGraphType(InTargetGraph);
	return (GraphType == GT_Ubergraph || GraphType == GT_Macro) && Super::IsCompatibleWithGraph(InTargetGraph);
}

FText UK2Node_DataLinkRequest::GetNodeTitle(ENodeTitleType::Type InTitleType) const
{
	return LOCTEXT("NodeTitle", "Execute Data Link");
}

bool UK2Node_DataLinkRequest::IsDeprecated() const
{
	return true;
}

FEdGraphNodeDeprecationResponse UK2Node_DataLinkRequest::GetDeprecationResponse(EEdGraphNodeDeprecationType InDeprecationType) const
{
	if (InDeprecationType == EEdGraphNodeDeprecationType::NodeTypeIsDeprecated)
	{
		FEdGraphNodeDeprecationResponse Response;
		Response.MessageType = EEdGraphNodeDeprecationMessageType::Warning;
		Response.MessageText = LOCTEXT("NodeDeprecated_Warning", "@@ is deprecated. Prefer creating a Data Link Executor Object and using that instead.");
		return Response;
	}
	return Super::GetDeprecationResponse(InDeprecationType);
}

void UK2Node_DataLinkRequest::ExpandNode(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph)
{
	UK2Node_AsyncDataLinkRequest* AsyncRequestNode = InCompilerContext.SpawnIntermediateNode<UK2Node_AsyncDataLinkRequest>(this, InSourceGraph);
	AsyncRequestNode->AllocateDefaultPins();

	InCompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *AsyncRequestNode->GetExecPin());
	InCompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *AsyncRequestNode->GetThenPin());
	InCompilerContext.MovePinLinksToIntermediate(*FindPinChecked(PN_DataLinkInstance, EGPD_Input), *AsyncRequestNode->GetDataLinkInstancePin());
	InCompilerContext.MovePinLinksToIntermediate(*FindPinChecked(PN_ExecutionContext, EGPD_Input), *AsyncRequestNode->GetExecutionContextPin());
	InCompilerContext.MovePinLinksToIntermediate(*FindPinChecked(PN_DataLinkSinkProvider, EGPD_Input), *AsyncRequestNode->GetDataLinkSinkProviderPin());
	InCompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UDataLinkEdNode::PN_Output, EGPD_Output), *AsyncRequestNode->GetOutputDataPin());

	UEnum* const ResultEnum = StaticEnum<EDataLinkExecutionResult>();

	UK2Node_SwitchEnum* SwitchNode = InCompilerContext.SpawnIntermediateNode<UK2Node_SwitchEnum>(this, InSourceGraph);
	SwitchNode->Enum = ResultEnum;
	SwitchNode->AllocateDefaultPins();

	// Connect Output Data and Result Enum value pins to the Switch Node
	{
		UEdGraphPin* OnOutputDataPin = AsyncRequestNode->GetOnOutputDataPin();
		UEdGraphPin* ExecutionResultPin = AsyncRequestNode->GetExecutionResultPin();
		check(OnOutputDataPin && ExecutionResultPin);

		OnOutputDataPin->MakeLinkTo(SwitchNode->GetExecPin());
		ExecutionResultPin->MakeLinkTo(SwitchNode->GetSelectionPin());
	}

	// Move Result Exec pins
	{
		UEdGraphPin* SuccessOutputPin = SwitchNode->FindPinChecked(ResultEnum->GetNameStringByValue(static_cast<uint64>(EDataLinkExecutionResult::Succeeded)), EGPD_Output);
		UEdGraphPin* FailureOutputPin = SwitchNode->FindPinChecked(ResultEnum->GetNameStringByValue(static_cast<uint64>(EDataLinkExecutionResult::Failed)), EGPD_Output);

		InCompilerContext.MovePinLinksToIntermediate(*FindPinChecked(PN_Success), *SuccessOutputPin);
		InCompilerContext.MovePinLinksToIntermediate(*FindPinChecked(PN_Failure), *FailureOutputPin);
	}

	BreakAllNodeLinks();
}

FName UK2Node_DataLinkRequest::GetCornerIcon() const
{
	return TEXT("Graph.Latent.LatentIcon");
}

void UK2Node_DataLinkRequest::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	UClass* ActionKey = GetClass();

	if (InActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(ActionKey);
		checkf(NodeSpawner, TEXT("Node spawner failed to create a valid Node"));
		InActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_DataLinkRequest::GetMenuCategory() const
{
	return LOCTEXT("MenuCategory", "Data Link");
}

#undef LOCTEXT_NAMESPACE
