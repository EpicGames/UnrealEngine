// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkEdGraph.h"
#include "DataLinkGraph.h"
#include "Nodes/DataLinkEdOutputNode.h"

UDataLinkEdGraph::UDataLinkEdGraph()
{
	OnGraphCompiledHandle = UDataLinkGraph::OnGraphCompiled().AddUObject(this, &UDataLinkEdGraph::OnGraphCompiled);
}

UDataLinkEdOutputNode* UDataLinkEdGraph::FindOutputNode() const
{
	UDataLinkEdOutputNode* OutputNode;
	if (Nodes.FindItemByClass(&OutputNode))
	{
		return OutputNode;
	}
	return nullptr;
}

void UDataLinkEdGraph::InitializeNodes()
{
	for (UEdGraphNode* Node : Nodes)
	{
		if (UDataLinkEdNode* DataLinkEdNode = Cast<UDataLinkEdNode>(Node))
		{
			DataLinkEdNode->UpdateMetadata();
			DataLinkEdNode->RefreshPins();
		}
	}
}

void UDataLinkEdGraph::BeginDestroy()
{
	Super::BeginDestroy();

	UDataLinkGraph::OnGraphCompiled().Remove(OnGraphCompiledHandle);
	OnGraphCompiledHandle.Reset();
}

bool UDataLinkEdGraph::IsEditorOnly() const
{
	return true;
}

void UDataLinkEdGraph::DirtyGraph()
{
	// ChangeId could be regenerated every time graph is dirtied,
	// but it only makes sense when it matches the compiled change id
	if (ChangeId == LastCompiledChangeId)
	{
		ChangeId = FGuid::NewGuid();
	}
}

bool UDataLinkEdGraph::IsCompiledGraphUpToDate() const
{
	return ChangeId == LastCompiledChangeId;
}

void UDataLinkEdGraph::OnGraphCompiled(UDataLinkGraph* InCompiledGraph)
{
	if (InCompiledGraph && InCompiledGraph->GetEdGraph() == this)
	{
		LastCompiledChangeId = ChangeId;
	}
}
