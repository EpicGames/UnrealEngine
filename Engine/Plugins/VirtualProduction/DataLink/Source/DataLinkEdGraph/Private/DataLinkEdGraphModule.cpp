// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkEdGraphLog.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogDataLinkEdGraph);

IMPLEMENT_MODULE(FDefaultModuleImpl, DataLinkEdGraph)
