// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkEdGraphSchema.h"
#include "Actions/DataLinkGraphAction_NewNativeNode.h"
#include "Actions/DataLinkGraphAction_NewScriptNode.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Blueprint/BlueprintSupport.h"
#include "DataLinkNode.h"
#include "EdGraph/EdGraph.h"
#include "Nodes/DataLinkEdNode.h"
#include "Nodes/DataLinkEdOutputNode.h"
#include "Nodes/Script/DataLinkScriptNode.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "DataLinkEdGraphSchema"

/** Pin Categories */
const FLazyName UDataLinkEdGraphSchema::PC_Data = TEXT("Data");

/** Pin Category Colors */
const FLinearColor UDataLinkEdGraphSchema::PCC_Data = FLinearColor::White;

namespace UE::DataLinkEdGraph::Private
{
	template<typename InActionType, typename... InArgTypes>
	TSharedRef<InActionType> AddAction(FGraphContextMenuBuilder& InContextMenuBuilder, const FString& InCategory, InArgTypes&&... InArgs)
	{
		TSharedRef<InActionType> Action = MakeShared<InActionType>(Forward<InArgTypes>(InArgs)...);
		Action->CosmeticUpdateRootCategory(FText::FromString(InCategory));
		InContextMenuBuilder.AddAction(Action);
		return Action;
	}
}

bool UDataLinkEdGraphSchema::IsConnectionLooping(const UEdGraphPin* InInputPin, const UEdGraphPin* InOutputPin) const
{ 
	// Inline size of 1 as most cases the nodes will be connected 1 to 1
	TArray<const UEdGraphNode*, TInlineAllocator<1>> NodesToCheck = { InInputPin->GetOwningNode() };

	const UEdGraphNode* OutputNode = InOutputPin->GetOwningNode();

	// Loop by starting with the Input Pin's Node and going in the direction opposite to the Input Pin direction (Output)
	// and see if we encounter the Output Pin's node
	while (!NodesToCheck.IsEmpty())
	{
		if (const UEdGraphNode* Node = NodesToCheck.Pop(EAllowShrinking::No))
		{
			if (Node == OutputNode)
			{
				// Output node detected! Looping found
				return true;
			}

			for (const UEdGraphPin* Pin : Node->Pins)
			{
				if (Pin && Pin->Direction == EGPD_Output)
				{
					for (const UEdGraphPin* LinkedPin : Pin->LinkedTo)
					{
						NodesToCheck.AddUnique(LinkedPin->GetOwningNode());
					}
				}
			}
		}
	}
	return false;
}

void UDataLinkEdGraphSchema::CreateDefaultNodesForGraph(UEdGraph& InGraph) const
{
	FGraphNodeCreator<UDataLinkEdOutputNode> NodeCreator(InGraph);
	UDataLinkEdOutputNode* OutputNode = NodeCreator.CreateNode();
	NodeCreator.Finalize();

	SetNodeMetaData(OutputNode, FNodeMetadata::DefaultGraphNode);
}

bool UDataLinkEdGraphSchema::ArePinsCompatible(const UEdGraphPin* InPinA, const UEdGraphPin* InPinB, const UClass* InCallingContext, bool bInIgnoreArray) const
{
	if (!InPinA || !InPinB)
	{
		return false;
	}

	auto IsDirectionCompatible = [](const UEdGraphPin* InPinA, const UEdGraphPin* InPinB)
		{
			return InPinA->Direction == EGPD_Input && InPinB->Direction == EGPD_Output;
		};

	if (!IsDirectionCompatible(InPinA, InPinB) && !IsDirectionCompatible(InPinB, InPinA))
	{
		return false;
	}

	return InPinA->PinType.PinCategory == InPinB->PinType.PinCategory
		&& InPinA->PinType.PinSubCategoryObject == InPinB->PinType.PinSubCategoryObject;
}

void UDataLinkEdGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& InContextMenuBuilder) const
{
	using namespace UE::DataLinkEdGraph;

	constexpr int32 Grouping = 0; 

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry")).Get();

	const FName MD_Category = TEXT("Category");

	// Add all node assets
	{
		const TMultiMap<FName, FString> TagValues =
		{
			{ FBlueprintTags::NativeParentClassPath, FObjectPropertyBase::GetExportPath(UDataLinkScriptNode::StaticClass()) },
		};

		TArray<FAssetData> ScriptNodeAssets;
		AssetRegistry.GetAssetsByTagValues(TagValues, ScriptNodeAssets);

		for (const FAssetData& ScriptNodeAsset : ScriptNodeAssets)
		{
			Private::AddAction<FDataLinkGraphAction_NewScriptNode>(InContextMenuBuilder
				, ScriptNodeAsset.GetTagValueRef<FString>(MD_Category)
				, ScriptNodeAsset
				, Grouping);
		}
	}

	const FName MD_Hidden = TEXT("Hidden");

	// Add all node native classes
	for (UClass* Class : TObjectRange<UClass>())
	{
		if (Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated) || Class->HasMetaData(MD_Hidden))
		{
			continue;
		}

		if (Class->HasAllClassFlags(CLASS_Native) && Class->IsChildOf<UDataLinkNode>())
		{
			Private::AddAction<FDataLinkGraphAction_NewNativeNode>(InContextMenuBuilder
				, Class->GetMetaData(MD_Category)
				, TSubclassOf<UDataLinkNode>(Class)
				, Grouping);
		}
	}
}

void UDataLinkEdGraphSchema::GetContextMenuActions(UToolMenu* InMenu, UGraphNodeContextMenuContext* InContext) const
{
	Super::GetContextMenuActions(InMenu, InContext);
}

const FPinConnectionResponse UDataLinkEdGraphSchema::CanCreateConnection(const UEdGraphPin* InSourcePin, const UEdGraphPin* InTargetPin) const
{
	// Make sure the pins are not on the same node
	if (InSourcePin->GetOwningNode() == InTargetPin->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionSameNode", "Both are on the same node"));
	}

	// Pin mismatch in Pin Category
	if (InSourcePin->PinType.PinCategory != InTargetPin->PinType.PinCategory)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("IncompatibleCategories", "Pin Types are not Compatible"));
	}

	// Compare the directions
	const UEdGraphPin* InputPin = nullptr;
	const UEdGraphPin* OutputPin = nullptr;

	if (!CategorizePinsByDirection(InSourcePin, InTargetPin, InputPin, OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("IncompatibleDirections", "Directions are not compatible"));
	}

	if (IsConnectionLooping(InputPin, OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionLoop", "Connection would cause loop"));
	}

	const bool bHasSourcePinLinks = !InSourcePin->LinkedTo.IsEmpty();
	const bool bHasTargetPinLinks = !InTargetPin->LinkedTo.IsEmpty();

	if (bHasSourcePinLinks && bHasTargetPinLinks)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_AB, LOCTEXT("ConnectionReplaceAB", "Replace existing connections"));
	}

	if (bHasSourcePinLinks)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_A, LOCTEXT("ConnectionReplaceA", "Replace existing connections"));
	}

	if (bHasTargetPinLinks)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_B, LOCTEXT("ConnectionReplaceB", "Replace existing connections"));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
}

FLinearColor UDataLinkEdGraphSchema::GetPinTypeColor(const FEdGraphPinType& InPinType) const
{
	if (InPinType.PinCategory == PC_Data)
	{
		return UDataLinkEdGraphSchema::PCC_Data;
	}
	return Super::GetPinTypeColor(InPinType);
}

void UDataLinkEdGraphSchema::GetGraphDisplayInformation(const UEdGraph& InGraph, FGraphDisplayInfo& OutDisplayInfo) const
{
	OutDisplayInfo.PlainName = FText::FromName(InGraph.GetFName());
	OutDisplayInfo.DisplayName = OutDisplayInfo.PlainName;
	OutDisplayInfo.Tooltip = LOCTEXT("GraphTooltip", "Graph used to determine how data is linked and flows from a source");
}

#undef LOCTEXT_NAMESPACE
