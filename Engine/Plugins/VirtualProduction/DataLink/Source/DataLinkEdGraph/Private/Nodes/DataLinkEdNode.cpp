// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/DataLinkEdNode.h"
#include "DataLinkEdGraphLog.h"
#include "DataLinkEdGraphSchema.h"
#include "DataLinkNode.h"
#include "DataLinkUtils.h"
#include "EdGraph/EdGraph.h"
#include "Engine/Engine.h"
#include "Templates/SubclassOf.h"

const FLazyName UDataLinkEdNode::MD_InvalidatesNode = TEXT("InvalidatesNode");
const FLazyName UDataLinkEdNode::PN_Output = TEXT("Output");

namespace UE::DataLinkEdGraph::Private
{
	bool IsMatchingPin(TNotNull<const UEdGraphPin*> InPin, const FDataLinkPin& InDataLinkPin)
	{
		return InPin->PinName == InDataLinkPin.Name
			&& InPin->PinType.PinCategory == UDataLinkEdGraphSchema::PC_Data
			&& InPin->PinType.PinSubCategoryObject == InDataLinkPin.Struct;
	}

	bool FindLinkedNode(const UEdGraphPin* InPin, const UDataLinkEdNode*& OutLinkedNode, const UEdGraphPin*& OutLinkedPin)
	{
		if (!InPin || InPin->LinkedTo.IsEmpty())
		{
			return false;
		}

		UEdGraphPin* LinkedPin = InPin->LinkedTo[0];
		if (!LinkedPin)
		{
			return false;
		}

		if (const UDataLinkEdNode* EdNode = Cast<UDataLinkEdNode>(LinkedPin->GetOwningNode()))
		{
			OutLinkedNode = EdNode;
			OutLinkedPin = LinkedPin;
			return true;
		}
		return false;
	}

	int32 GetDataPinCount(const UDataLinkEdNode& InEdNode)
	{
		int32 DataPinCount = 0;
		for (UEdGraphPin* Pin : InEdNode.Pins)
		{
			if (Pin && Pin->PinType.PinCategory == UDataLinkEdGraphSchema::PC_Data)
			{
				++DataPinCount;
			}
		}
		return DataPinCount;
	}
}

void UDataLinkEdNode::SetTemplateNodeClass(TSubclassOf<UDataLinkNode> InNodeClass, bool bInReconstructNode)
{
	UObject* Node = TemplateNode;
	if (UE::DataLink::ReplaceObject(Node, this, InNodeClass))
	{
		TemplateNode = Cast<UDataLinkNode>(Node);

		if (bInReconstructNode)
		{
			ReconstructNode();
		}
	}
}

void UDataLinkEdNode::ForEachPinConnection(TFunctionRef<void(const UEdGraphPin&, const UDataLinkEdNode&, const UEdGraphPin&)> InFunction) const
{
	for (const UEdGraphPin* Pin : Pins)
	{
		const UDataLinkEdNode* LinkedNode;
		const UEdGraphPin* LinkedPin;

		if (UE::DataLinkEdGraph::Private::FindLinkedNode(Pin, LinkedNode, LinkedPin))
		{
			InFunction(*Pin, *LinkedNode, *LinkedPin);
		}
	}
}

bool UDataLinkEdNode::RequiresPinRecreation() const
{
	const int32 DataPinCount = UE::DataLinkEdGraph::Private::GetDataPinCount(*this);
	if (!TemplateNode)
	{
		// If a template node is not set, but there are still Data Pins in place, they need to be cleared off
		return DataPinCount != 0;
	}

	TArray<FDataLinkPin> InputPins;
	TArray<FDataLinkPin> OutputPins;
	TemplateNode->BuildPins(InputPins, OutputPins);

	// Require pin recreation if the number of 'data' pins mismatch the total number of input and output pins of the Template
	if (DataPinCount != InputPins.Num() + OutputPins.Num())
	{
		return true;
	}

	auto HasMismatchingPin = [This=this](TConstArrayView<FDataLinkPin> InTemplatePins, EEdGraphPinDirection InDirection)
		{
			for (const FDataLinkPin& TemplatePin : InTemplatePins)
			{
				const UEdGraphPin* const FoundPin = This->FindPin(TemplatePin.Name, InDirection);

				// Require pin recreation if there's no found pin for the template data link pin or if it mismatches from it
				if (!FoundPin || !UE::DataLinkEdGraph::Private::IsMatchingPin(FoundPin, TemplatePin))
				{
					return true;
				}
			}
			return false;
		};

	return HasMismatchingPin(InputPins, EGPD_Input) || HasMismatchingPin(OutputPins, EGPD_Output);
}

void UDataLinkEdNode::AutowireNewNode(UEdGraphPin* InFromPin)
{
	if (!InFromPin)
	{
		return;
	}

	const UEdGraphSchema* const Schema = GetSchema();
	if (!Schema)
	{
		return;
	}

	UEdGraphPin* TargetPin = nullptr;

	// Iterate in reverse so that the most compatible pin end up being the first pins over the last
	for (UEdGraphPin* Pin : ReverseIterate(Pins))
	{
		if (Pin && Schema->ArePinsCompatible(Pin, InFromPin))
		{
			TargetPin = Pin;

			// If the pin names match, the best possible pin has been found
			if (Pin->PinName == InFromPin->PinName)
			{
				break;
			}
		}
	}

	if (!TargetPin)
	{
		return;
	}

	if (Schema->TryCreateConnection(InFromPin, TargetPin))
	{
		InFromPin->GetOwningNode()->NodeConnectionListChanged();
	}
	else if (Schema->TryCreateConnection(TargetPin, InFromPin))
	{
		NodeConnectionListChanged();
	}
}

void UDataLinkEdNode::ReconstructNode()
{
	Super::ReconstructNode();
	UpdateMetadata();
	RefreshPins();
}

bool UDataLinkEdNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* InSchema) const
{
	return InSchema && InSchema->IsA<UDataLinkEdGraphSchema>();
}

FText UDataLinkEdNode::GetNodeTitle(ENodeTitleType::Type InTitleType) const
{
	return NodeMetadata.GetDisplayName();
}

FText UDataLinkEdNode::GetTooltipText() const
{
	return NodeMetadata.GetTooltipText();
}

void UDataLinkEdNode::PinConnectionListChanged(UEdGraphPin* InPin)
{
	Super::PinConnectionListChanged(InPin);
	NotifyNodeChanged();
}

void UDataLinkEdNode::PostLoad()
{
	Super::PostLoad();
	UpdateMetadata();
}

void UDataLinkEdNode::PostEditUndo()
{
	Super::PostEditUndo();
	UpdateMetadata();
	NotifyNodeChanged();
}

void UDataLinkEdNode::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(InPropertyChangedEvent);

	// Iterate over each property to see if there's a property that invalidates the node and requires node refresh
	for (FProperty* Property : InPropertyChangedEvent.PropertyChain)
	{
		if (Property && Property->HasMetaData(UDataLinkEdNode::MD_InvalidatesNode))
		{
			UpdateMetadata();
			RefreshPins();
			break;
		}
	}

	NotifyNodeChanged();
}

void UDataLinkEdNode::UpdateMetadata()
{
	NodeMetadata = FDataLinkNodeMetadata();

	if (TemplateNode)
	{
		TemplateNode->BuildMetadata(NodeMetadata);
	}
	else
	{
		NodeMetadata
			.SetDisplayName(GetClass()->GetDisplayNameText())
			.SetTooltipText(GetClass()->GetToolTipText());
	}
}

void UDataLinkEdNode::NotifyNodeChanged()
{
	if (UEdGraph* Graph = GetGraph())
	{
		Graph->NotifyNodeChanged(this);
	}
}

void UDataLinkEdNode::RefreshPins()
{
	if (!RequiresPinRecreation())
	{
		return;
	}

	TArray<FDataLinkPin> InputPins;
	TArray<FDataLinkPin> OutputPins;

	// Build Data Link Node Pins and create them on the Ed Node
	if (TemplateNode)
	{
		TemplateNode->BuildPins(InputPins, OutputPins);
	}

	bool bPinsChanged = false;
	bPinsChanged |= SyncPins(EGPD_Input, InputPins);
	bPinsChanged |= SyncPins(EGPD_Output, OutputPins);

	// Refresh the UI for the graph so the pin changes show up
	if (bPinsChanged)
	{
		NotifyNodeChanged();
	}
}

bool UDataLinkEdNode::SyncPins(EEdGraphPinDirection InPinDirection, TConstArrayView<FDataLinkPin> InTemplatePins)
{
	bool bPinsChanged = false;

	// Remove pins that are no longer present in the provided template pins, or that mismatch the template pin
	TArray<UEdGraphPin*> PinsCopy = Pins;
	for (UEdGraphPin* Pin : PinsCopy)
	{
		if (!Pin || Pin->Direction != InPinDirection)
		{
			continue;
		}

		const FDataLinkPin* const TemplatePin = InTemplatePins.FindByKey(Pin->PinName);
		if (!TemplatePin || !UE::DataLinkEdGraph::Private::IsMatchingPin(Pin, *TemplatePin))
		{
			RemovePin(Pin);
			bPinsChanged = true;
		}
	}

	// Create missing pin links
	for (const FDataLinkPin& DataLinkPin : InTemplatePins)
	{
		// No need to add if Pin already is there
		if (!FindPin(DataLinkPin.Name, InPinDirection))
		{
			UEdGraphPin* const Pin = CreatePin(InPinDirection, UDataLinkEdGraphSchema::PC_Data, ConstCast(DataLinkPin.Struct), DataLinkPin.Name);
			check(Pin);
			Pin->PinFriendlyName = DataLinkPin.GetDisplayName();
			bPinsChanged = true;
		}
	}

	return bPinsChanged;
}
