// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/DataLinkEdOutputNode.h"
#include "DataLinkEdGraphSchema.h"

UEdGraphPin* UDataLinkEdOutputNode::GetOutputResultPin() const
{
	return FindPin(UDataLinkEdNode::PN_Output, EEdGraphPinDirection::EGPD_Input);
}

FLinearColor UDataLinkEdOutputNode::GetNodeTitleColor() const
{
	return FLinearColor::Red;
}

void UDataLinkEdOutputNode::AllocateDefaultPins()
{
	// The finishing output node only has input pin.
	// Named as "Output" to indicate result
	CreatePin(EGPD_Input, UDataLinkEdGraphSchema::PC_Data, UDataLinkEdNode::PN_Output);
}
