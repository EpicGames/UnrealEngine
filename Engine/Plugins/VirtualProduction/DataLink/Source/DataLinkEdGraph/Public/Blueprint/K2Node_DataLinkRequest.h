// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"
#include "K2Node_DataLinkRequest.generated.h"

class UDataLinkGraph;

UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "Data Link Request deprecated. Use Data Link Executor Object instead.") UK2Node_DataLinkRequest : public UK2Node
{
	GENERATED_BODY()

public:
	static const FLazyName PN_DataLinkInstance;
	static const FLazyName PN_ExecutionContext;
	static const FLazyName PN_DataLinkSinkProvider;
	static const FLazyName PN_Success;
	static const FLazyName PN_Failure;

	//~ Begin UEdGraphNode
	virtual void AllocateDefaultPins() override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* InTargetGraph) const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type InTitleType) const override;
	virtual bool IsDeprecated() const override;
	virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType InDeprecationType) const override;
	//~ End UEdGraphNode

	//~ Begin UK2Node
	virtual void ExpandNode(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph) override;
	virtual FName GetCornerIcon() const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	//~ End UK2Node
};
