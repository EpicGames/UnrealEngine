// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h"
#include "DataLinkEdGraph.generated.h"

class UDataLinkEdOutputNode;
class UDataLinkGraph;

UCLASS(MinimalAPI)
class UDataLinkEdGraph : public UEdGraph
{
    GENERATED_BODY()

public:
	UDataLinkEdGraph();

    /**
     * Finds the 'cosmetic' output node
     * @see UDataLinkEdOutputNode
     */
    DATALINKEDGRAPH_API UDataLinkEdOutputNode* FindOutputNode() const;

	/** Initializes all the nodes in this graph, recreating pins of outdated nodes to ensure these are up-to-date with their templates */
	DATALINKEDGRAPH_API void InitializeNodes();

	//~ Begin UObject
	DATALINKEDGRAPH_API virtual void BeginDestroy() override;
	DATALINKEDGRAPH_API virtual bool IsEditorOnly() const override;
	//~ End UObject

	DATALINKEDGRAPH_API void DirtyGraph();

	DATALINKEDGRAPH_API bool IsCompiledGraphUpToDate() const;

private:
	void OnGraphCompiled(UDataLinkGraph* InCompiledGraph);

	/**
	 * Represents id of graph changes that haven't been compiled yet.
	 * Note: if a change occurs, this id will be regenerated only if the Last Compiled Change Id matches the current change id.
	 */
	UPROPERTY()
	FGuid ChangeId;

	/** The Change Id that was last compiled */
	UPROPERTY()
	FGuid LastCompiledChangeId;

	FDelegateHandle OnGraphCompiledHandle;
};
