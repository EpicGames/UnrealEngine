// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "DataLinkEdGraphSchema.generated.h"

UCLASS(MinimalAPI)
class UDataLinkEdGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()

public:
	/** Pin Categories */
	DATALINKEDGRAPH_API static const FLazyName PC_Data;

	/** Pin Category Colors */
	DATALINKEDGRAPH_API static const FLinearColor PCC_Data;

	/** Check whether connecting these pins would cause a loop */
	bool IsConnectionLooping(const UEdGraphPin* InInputPin, const UEdGraphPin* InOutputPin) const;

	//~ Begin UEdGraphSchema
	virtual void CreateDefaultNodesForGraph(UEdGraph& InGraph) const override;
	virtual bool ArePinsCompatible(const UEdGraphPin* InPinA, const UEdGraphPin* InPinB, const UClass* InCallingContext = nullptr, bool bInIgnoreArray = false) const override;
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& InContextMenuBuilder) const override;
	virtual void GetContextMenuActions(UToolMenu* InMenu, UGraphNodeContextMenuContext* InContext) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* InSourcePin, const UEdGraphPin* InTargetPin) const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& InPinType) const override;
	virtual void GetGraphDisplayInformation(const UEdGraph& InGraph, FGraphDisplayInfo& OutDisplayInfo) const override;
	//~ End UEdGraphSchema
};
