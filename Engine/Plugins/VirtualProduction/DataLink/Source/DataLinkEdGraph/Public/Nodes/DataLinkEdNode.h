// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNodeMetadata.h"
#include "EdGraph/EdGraphNode.h"
#include "DataLinkEdNode.generated.h"

class UDataLinkNode;
struct FDataLinkPin;
template <typename T> class TSubclassOf;

UCLASS(MinimalAPI)
class UDataLinkEdNode : public UEdGraphNode
{
	GENERATED_BODY()

public:
	DATALINKEDGRAPH_API static const FLazyName MD_InvalidatesNode;
	DATALINKEDGRAPH_API static const FLazyName PN_Output;

	DATALINKEDGRAPH_API void SetTemplateNodeClass(TSubclassOf<UDataLinkNode> InNodeClass, bool bInReconstructNode = true);

	DATALINKEDGRAPH_API void ForEachPinConnection(TFunctionRef<void(const UEdGraphPin&, const UDataLinkEdNode&, const UEdGraphPin&)> InFunction) const;

	UDataLinkNode* GetTemplateNode() const
	{
		return TemplateNode;
	}

	//~ Begin UEdGraphNode
	DATALINKEDGRAPH_API virtual void AutowireNewNode(UEdGraphPin* InFromPin) override;
	DATALINKEDGRAPH_API virtual void ReconstructNode() override;
	DATALINKEDGRAPH_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* InSchema) const override;
	DATALINKEDGRAPH_API virtual FText GetNodeTitle(ENodeTitleType::Type InTitleType) const override;
	DATALINKEDGRAPH_API virtual FText GetTooltipText() const override;
	DATALINKEDGRAPH_API virtual void PinConnectionListChanged(UEdGraphPin* InPin) override;
	//~ End UEdGraphNode

	//~ Begin UObject
	DATALINKEDGRAPH_API virtual void PostLoad() override;
	DATALINKEDGRAPH_API virtual void PostEditUndo() override;
	DATALINKEDGRAPH_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;
	//~ End UObject

	static FName GetTemplateNodePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UDataLinkEdNode, TemplateNode);
	}

	void UpdateMetadata();

	DATALINKEDGRAPH_API void NotifyNodeChanged();

	/**
 	 * Checks against the current Template Node to determine if the Ed Node's structure matches the Template Node's
 	 * @return true if the structures mismatch and requires node reconstruction
 	 */
	DATALINKEDGRAPH_API virtual bool RequiresPinRecreation() const;

	/** Destroys existing pins and recreates new ones based on the underlying Template Node while also restoring pin links where available */
	void RefreshPins();

private:
	/**
	 * Syncs the node pins to match the given data link pins
	 * @param InPinDirection the direction of pins to consider
	 * @param InTemplatePins the pins to match
	 * @return true if the pin structure was changed
	 */
	bool SyncPins(EEdGraphPinDirection InPinDirection, TConstArrayView<FDataLinkPin> InTemplatePins);

	/** Underlying Data Link Node this Editor Node represents */
	UPROPERTY(VisibleAnywhere, Instanced, Category="Data Link")
	TObjectPtr<UDataLinkNode> TemplateNode;

	/** Cached metadata from the Template Node */
	FDataLinkNodeMetadata NodeMetadata;
};
