// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkEdNode.h"
#include "DataLinkEdOutputNode.generated.h"

/**
 * A 'cosmetic' output node.
 * This is considered cosmetic because it does not get compiled in and
 * is used only to iterate through the nodes that should be considered for compilation.
 */
UCLASS(MinimalAPI, DisplayName="Output")
class UDataLinkEdOutputNode : public UDataLinkEdNode
{
	GENERATED_BODY()

public:
	/** Retrieves the output result pin */
	DATALINKEDGRAPH_API UEdGraphPin* GetOutputResultPin() const;

	//~ Begin UDataLinkEdNode
	virtual bool RequiresPinRecreation() const override { return false; }
	//~ End UDataLinkEdNode

	//~ Begin UEdGraphNode
	DATALINKEDGRAPH_API virtual FLinearColor GetNodeTitleColor() const;
	DATALINKEDGRAPH_API virtual void AllocateDefaultPins() override;
	virtual bool CanDuplicateNode() const override { return false; }
	virtual bool CanUserDeleteNode() const override { return false; }
	virtual bool GetCanRenameNode() const override { return false; }
	//~ End UEdGraphNode
};
