// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataLinkEditor : ModuleRules
{
    public DataLinkEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "ApplicationCore",
                "AssetDefinition",
                "DataLink",
                "DataLinkEdGraph",
                "Engine",
                "GraphEditor",
                "PropertyEditor",
                "Slate",
                "SlateCore",
                "StructUtilsEditor",
                "ToolMenus",
                "UnrealEd",
            }
        );
    }
}
