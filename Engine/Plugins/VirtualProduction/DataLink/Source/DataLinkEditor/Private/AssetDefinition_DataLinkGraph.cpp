// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DataLinkGraph.h"
#include "DataLinkGraph.h"
#include "DataLinkGraphAssetEditor.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_DataLinkGraph"

FText UAssetDefinition_DataLinkGraph::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Data Link Graph");
}

FLinearColor UAssetDefinition_DataLinkGraph::GetAssetColor() const
{
	return FLinearColor(FColor(64, 130, 109));
}

TSoftClassPtr<UObject> UAssetDefinition_DataLinkGraph::GetAssetClass() const
{
	return UDataLinkGraph::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DataLinkGraph::GetAssetCategories() const
{
	return MakeArrayView(&EAssetCategoryPaths::Misc, 1);
}

EAssetCommandResult UAssetDefinition_DataLinkGraph::OpenAssets(const FAssetOpenArgs& InOpenArgs) const
{
	check(GEditor);
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	check(AssetEditorSubsystem);

	for (UDataLinkGraph* DataLinkGraph : InOpenArgs.LoadObjects<UDataLinkGraph>())
	{
		UDataLinkGraphAssetEditor* AssetEditor = NewObject<UDataLinkGraphAssetEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);
		AssetEditor->Initialize(DataLinkGraph);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
