// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphCompiler.h"
#include "DataLinkEditorLog.h"
#include "DataLinkGraph.h"
#include "DataLinkNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Nodes/DataLinkEdNode.h"
#include "Nodes/DataLinkEdOutputNode.h"

FDataLinkGraphCompiler::FDataLinkGraphCompiler(UDataLinkGraph* InDataLinkGraph)
	: DataLinkGraph(InDataLinkGraph)
{
}

UE::DataLink::EGraphCompileStatus FDataLinkGraphCompiler::Compile()
{
	using namespace UE::DataLink;

	if (!DataLinkGraph)
	{
		UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. Data Link Graph is invalid.");
		return EGraphCompileStatus::Error;
	}

	DataLinkEdGraph = Cast<UDataLinkEdGraph>(DataLinkGraph->GetEdGraph());
	if (!DataLinkEdGraph)
	{
		UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. Data Link Ed Graph is invalid in graph '%ls'", *DataLinkGraph->GetName());
		return EGraphCompileStatus::Error;
	}

	CleanExistingGraph();

	if (!CompileNodes())
	{
		// Compilation failed. Clean graph again from what had started compiling
    	CleanExistingGraph();
    	return EGraphCompileStatus::Error;
	}

	UDataLinkGraph::OnGraphCompiledDelegate.Broadcast(DataLinkGraph);
	return EGraphCompileStatus::UpToDate;
}

void FDataLinkGraphCompiler::CleanExistingGraph()
{
	DataLinkGraph->InputNodes.Reset();
	DataLinkGraph->OutputNode = nullptr;
	DataLinkGraph->Nodes.Reset();
}

bool FDataLinkGraphCompiler::CompileNodes()
{
	if (DataLinkEdGraph->Nodes.IsEmpty())
	{
		// Nothing further to compile. Unknown Data Link Graph
		UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. No graph node were found! Graph: %ls", *DataLinkGraph->GetName());
		return false;
	}

	// Step 1: Compile the nodes and fill in the Ed to Compiled Node map
	if (!CreateCompiledNodes())
	{
		return false;
	}

	// Step 2: Set the node links by finding the Editor link and using the map to find the compiled node
	LinkNodes();

	// Step 3: Find and set the inputs and output nodes
	SetInputOutputNodes();

	return true;
}

bool FDataLinkGraphCompiler::CreateCompiledNodes()
{
	UDataLinkEdOutputNode* const OutputNode = DataLinkEdGraph->FindOutputNode();
	if (!OutputNode)
	{
		UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. Output Node was not valid! Graph: %ls"
			, *DataLinkGraph->GetName());
		return false;
	}

	const UEdGraphPin* OutputResultPin = OutputNode->GetOutputResultPin();
	if (!OutputResultPin || OutputResultPin->LinkedTo.IsEmpty() || !OutputResultPin->LinkedTo[0])
	{
		UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. No output result pin was found or connected in graph '%ls'", *DataLinkGraph->GetName());
		return false;
	}

	// The node that the cosmetic output node is connected to is the actual output node at runtime
	const UEdGraphPin* OutputPin = OutputResultPin->LinkedTo[0];
	if (!ensure(OutputPin))
	{
		UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. Output pin connected to the output result node was invalid in graph '%ls'", *DataLinkGraph->GetName());
		return false;
	}

	OutputEdNode = Cast<UDataLinkEdNode>(OutputPin->GetOwningNode());
	OutputPinName = OutputPin->PinName;

	if (!ensure(OutputEdNode))
	{
		UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. Output node connected to the output result node was not a data link node in graph '%ls'", *DataLinkGraph->GetName());
		return false;
	}

	EdToCompiledMap.Empty(DataLinkEdGraph->Nodes.Num());

	TArray<const UDataLinkEdNode*, TInlineAllocator<1>> EdNodesRemaining;
	EdNodesRemaining.Add(OutputEdNode);

	const auto AddLinkedNodes =
		[&EdToCompiledMap = EdToCompiledMap, &EdNodesRemaining](const UEdGraphPin& InPin, const UDataLinkEdNode& InLinkedNode, const UEdGraphPin& InLinkedPin)
		{
			// Skip pin connections that aren't input
			// Skip nodes that have already been compiled
			if (InPin.Direction == EGPD_Input && !EdToCompiledMap.Contains(&InLinkedNode))
			{
				// Add the node only once (in cases where multiple pins are connected to one node)
				EdNodesRemaining.AddUnique(&InLinkedNode);
			}
		};

	while (!EdNodesRemaining.IsEmpty())
	{
		const UDataLinkEdNode* EdNode = EdNodesRemaining.Pop();
		if (!EdNode)
		{
			continue;
		}

		if (UDataLinkNode* TemplateNode = EdNode->GetTemplateNode())
		{
			EdToCompiledMap.Add(EdNode, CompileNode(TemplateNode));
		}
		else
		{
			UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. EdNode '%ls' did not have a valid Template Node! Graph: %ls", *EdNode->GetName(), *DataLinkGraph->GetName());
			return false;
		}

		// Add connected nodes in the input direction
		EdNode->ForEachPinConnection(AddLinkedNodes);
	}

	if (EdToCompiledMap.IsEmpty())
	{
		UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. No nodes were considered for compilation in graph '%ls'", *DataLinkGraph->GetName());
		return false;	
	}

	// Populate the Nodes array
	DataLinkGraph->Nodes.Reserve(EdToCompiledMap.Num());
	for (const TPair<const UDataLinkEdNode*, UDataLinkNode*>& Pair : EdToCompiledMap)
	{
		DataLinkGraph->Nodes.Add(Pair.Value);
	}

	return true;
}

UDataLinkNode* FDataLinkGraphCompiler::CompileNode(UDataLinkNode* InTemplateNode)
{
	check(InTemplateNode);

	UDataLinkNode* CompiledNode = NewObject<UDataLinkNode>(DataLinkGraph
		, InTemplateNode->GetClass()
		, NAME_None
		, RF_NoFlags
		, InTemplateNode);

	// Build Pins
	{
		TArray<FDataLinkPin> InputPins;
		TArray<FDataLinkPin> OutputPins;

		CompiledNode->BuildPins(InputPins, OutputPins);

		CompiledNode->InputPins = MoveTemp(InputPins);
		CompiledNode->OutputPins = MoveTemp(OutputPins);
	}

	CompiledNode->FixupNode();
	return CompiledNode;
}

void FDataLinkGraphCompiler::LinkNodes()
{
	for (UEdGraphNode* EdNodeRaw : DataLinkEdGraph->Nodes)
	{
		UDataLinkEdNode* EdNode = Cast<UDataLinkEdNode>(EdNodeRaw);
		if (!EdNode)
		{
			continue;
		}

		UDataLinkNode* CompiledNode = FindCompiledNode(EdNode);
		if (!CompiledNode)
		{
			continue;
		}

		// Update Inputs and Output Pins to have the compiled connected node
		EdNode->ForEachPinConnection(
			[this, CompiledNode](const UEdGraphPin& InPin, const UDataLinkEdNode& InLinkedNode, const UEdGraphPin& InLinkedPin)
			{
				const bool bIsInputPin = InPin.Direction == EGPD_Input;

				FDataLinkPin* const Pin = bIsInputPin
					? CompiledNode->InputPins.FindByKey(InPin.PinName)
					: CompiledNode->OutputPins.FindByKey(InPin.PinName);

				if (!Pin)
				{
					return;
				}

				if (UDataLinkNode* LinkedNode = FindCompiledNode(&InLinkedNode))
				{
					Pin->LinkedNode = LinkedNode;

					// Link Index is the opposite since Input Pins are connected to Output Pins
					Pin->LinkedIndex = !bIsInputPin
						? LinkedNode->InputPins.IndexOfByKey(InLinkedPin.PinName)
						: LinkedNode->OutputPins.IndexOfByKey(InLinkedPin.PinName);
				}
				else
				{
					Pin->LinkedNode = nullptr;
					Pin->LinkedIndex = INDEX_NONE;
				}
			});
	}
}

bool FDataLinkGraphCompiler::SetInputOutputNodes()
{
	// at this point in compilation the map shouldn't be empty
	check(!EdToCompiledMap.IsEmpty());

	UDataLinkNode* OutputNode = FindCompiledNode(OutputEdNode);
	if (!ensureAlways(OutputNode))
	{
		UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. Output Node was not found in graph '%ls'"
			, *DataLinkGraph->GetName());
		return false;
	}

	DataLinkGraph->OutputNode = OutputNode;
	DataLinkGraph->OutputPinName = OutputPinName;

	// Recursively adds all the graph's entry nodes (i.e. nodes with input pins that are not connected to other nodes)
	AddGraphEntryNodes(OutputNode);

	if (DataLinkGraph->InputNodes.IsEmpty())
	{
		UE_LOGF(LogDataLinkEditor, Error, "Compilation failed. Input nodes could not be determined in graph '%ls'", *DataLinkGraph->GetName())
		return false;
	}

	return true;
}

void FDataLinkGraphCompiler::AddGraphEntryNodes(const UDataLinkNode* InNode)
{
	TConstArrayView<FDataLinkPin> InputPins = InNode->GetInputPins();

	// Node with no input pins are considered entry nodes
	if (InputPins.IsEmpty())
	{
		DataLinkGraph->InputNodes.AddUnique(const_cast<UDataLinkNode*>(InNode));
		return;
	}

	bool bAddedInputNode = false;

	for (const FDataLinkPin& InputPin : InputPins)
	{
		// If there is a linked node to follow, this node is not the entry node for this pin path
		if (InputPin.LinkedNode)
		{
			AddGraphEntryNodes(InputPin.LinkedNode);
		}
		else if (!bAddedInputNode)
		{
			// Avoid adding the same node multiple times if multiple of its input pins are 'opened' (i.e. not linked to other node)
			DataLinkGraph->InputNodes.AddUnique(const_cast<UDataLinkNode*>(InNode));
			bAddedInputNode = true;
		}
	}
}

UDataLinkNode* FDataLinkGraphCompiler::FindCompiledNode(const UDataLinkEdNode* InEdNode) const
{
	if (UDataLinkNode* const* FoundNode = EdToCompiledMap.Find(InEdNode))
	{
		return *FoundNode;
	}
	return nullptr;
}
