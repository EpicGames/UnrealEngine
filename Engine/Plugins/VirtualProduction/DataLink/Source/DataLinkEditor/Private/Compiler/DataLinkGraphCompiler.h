// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkEdGraph.h"

namespace UE::DataLink
{
	enum class EGraphCompileStatus : uint8;
}

class UDataLinkEdNode;
class UDataLinkGraph;
class UDataLinkNode;
enum class EDataLinkGraphCompileStatus : uint8;

/** Compiles a data link graph's ed graph and writes its results to the runtime data of the data link graph */
class FDataLinkGraphCompiler
{
public:
	explicit FDataLinkGraphCompiler(UDataLinkGraph* InDataLinkGraph);

	UE::DataLink::EGraphCompileStatus Compile();

private:
	void CleanExistingGraph();

	bool CompileNodes();

	bool CreateCompiledNodes();

	UDataLinkNode* CompileNode(UDataLinkNode* InTemplateNode);

	void LinkNodes();

	bool SetInputOutputNodes();

	void AddGraphEntryNodes(const UDataLinkNode* InNode);

	UDataLinkNode* FindCompiledNode(const UDataLinkEdNode* InEdNode) const;

	TObjectPtr<UDataLinkGraph> DataLinkGraph;

	TObjectPtr<UDataLinkEdGraph> DataLinkEdGraph;

	TObjectPtr<const UDataLinkEdNode> OutputEdNode;

	/**
	 * Pin name of the output ed node connecting to the cosmetic output node
	 * Indicating that this pin is the pin that will be containing the graph output data
	 */
	FName OutputPinName;

	/** Map of the Editor Node to its Compiled Node */
	TMap<const UDataLinkEdNode*, UDataLinkNode*> EdToCompiledMap;
};
