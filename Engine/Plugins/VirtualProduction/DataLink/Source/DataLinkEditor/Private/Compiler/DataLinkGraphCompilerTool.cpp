// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphCompilerTool.h"
#include "DataLinkEditorStyle.h"
#include "DataLinkEnums.h"
#include "DataLinkGraphAssetEditor.h"
#include "DataLinkGraphAssetToolkit.h"
#include "DataLinkGraphCommands.h"
#include "DataLinkGraphCompiler.h"
#include "DataLinkGraphEditorMenuContext.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "DataLinkGraphCompilerToolkit"

namespace UE::DataLink::Editor::Private
{
	const TMap<EGraphCompileStatus, FLazyName> CompileStatusBackgrounds =
	{
		{ EGraphCompileStatus::Unknown , TEXT("CompileStatus.Background.Unknown") },
		{ EGraphCompileStatus::Warning , TEXT("CompileStatus.Background.Warning") },
		{ EGraphCompileStatus::Error   , TEXT("CompileStatus.Background.Error")  },
		{ EGraphCompileStatus::UpToDate, TEXT("CompileStatus.Background.Good")   },
	};

	const TMap<EGraphCompileStatus, FLazyName> CompileStatusOverlays =
	{
		{ EGraphCompileStatus::Unknown , TEXT("CompileStatus.Overlay.Unknown") },
		{ EGraphCompileStatus::Warning , TEXT("CompileStatus.Overlay.Warning") },
		{ EGraphCompileStatus::Error   , TEXT("CompileStatus.Overlay.Error")  },
		{ EGraphCompileStatus::UpToDate, TEXT("CompileStatus.Overlay.Good")   },
	};
}

void FDataLinkGraphCompilerTool::ExtendMenu(UToolMenu* InMenu)
{
	FToolMenuSection& CompilerSection = InMenu->FindOrAddSection(TEXT("Compile")
		, TAttribute<FText>()
		, FToolMenuInsert(TEXT("Asset"), EToolMenuInsertType::After));

	CompilerSection.AddDynamicEntry(TEXT("Compiler")
		, FNewToolMenuSectionDelegate::CreateStatic(&FDataLinkGraphCompilerTool::ExtendDynamicCompilerSection));
}

FDataLinkGraphCompilerTool::FDataLinkGraphCompilerTool(UDataLinkGraphAssetEditor* InAssetEditor)
	: AssetEditor(InAssetEditor)
	, LastCompiledStatus(UE::DataLink::EGraphCompileStatus::Unknown)
{
	UDataLinkEdGraph* EdGraph = AssetEditor->GetDataLinkEdGraph();
	if (EdGraph && EdGraph->IsCompiledGraphUpToDate())
	{
		LastCompiledStatus = UE::DataLink::EGraphCompileStatus::UpToDate;
	}
}

void FDataLinkGraphCompilerTool::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	const FDataLinkGraphCommands& GraphCommands = FDataLinkGraphCommands::Get();
	InCommandList->MapAction(GraphCommands.Compile, FExecuteAction::CreateSP(this, &FDataLinkGraphCompilerTool::Compile));
}

void FDataLinkGraphCompilerTool::Compile()
{
	FDataLinkGraphCompiler Compiler(AssetEditor->GetDataLinkGraph());
	LastCompiledStatus = Compiler.Compile();
}

void FDataLinkGraphCompilerTool::ExtendDynamicCompilerSection(FToolMenuSection& InSection)
{
	UDataLinkGraphEditorMenuContext* MenuContext = InSection.FindContext<UDataLinkGraphEditorMenuContext>();
	if (!MenuContext)
	{
		return;
	}

	TSharedPtr<FDataLinkGraphAssetToolkit> Toolkit = MenuContext->ToolkitWeak.Pin();
	if (!Toolkit.IsValid())
	{
		return;
	}

	const FDataLinkGraphCommands& GraphCommands = FDataLinkGraphCommands::Get();

	InSection.AddEntry(FToolMenuEntry::InitToolBarButton(GraphCommands.Compile
		, TAttribute<FText>()
		, TAttribute<FText>()
		, TAttribute<FSlateIcon>::CreateSP(&Toolkit->GetCompilerTool(), &FDataLinkGraphCompilerTool::GetCompileIcon)));
}

FSlateIcon FDataLinkGraphCompilerTool::GetCompileIcon() const
{
	using namespace UE::DataLink;

	EGraphCompileStatus CompileStatus = LastCompiledStatus;

	UDataLinkEdGraph* EdGraph = AssetEditor->GetDataLinkEdGraph();
	if (EdGraph && !EdGraph->IsCompiledGraphUpToDate())
	{
		CompileStatus = EGraphCompileStatus::Unknown;
	}

	const FName Background = Editor::Private::CompileStatusBackgrounds[CompileStatus]; 
	const FName Overlay = Editor::Private::CompileStatusOverlays[CompileStatus]; 

	return FSlateIcon(FDataLinkEditorStyle::Get().GetStyleSetName(), Background, Background, Overlay);
}

#undef LOCTEXT_NAMESPACE
