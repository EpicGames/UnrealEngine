// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

namespace UE::DataLink
{
	enum class EGraphCompileStatus : uint8;
}

class FUICommandList;
class UDataLinkGraphAssetEditor;
class UToolMenu;
struct FSlateIcon;
struct FToolMenuSection;

class FDataLinkGraphCompilerTool : public TSharedFromThis<FDataLinkGraphCompilerTool>
{
public:
	static void ExtendMenu(UToolMenu* InMenu);

	explicit FDataLinkGraphCompilerTool(UDataLinkGraphAssetEditor* InAssetEditor);

	void BindCommands(const TSharedRef<FUICommandList>& InCommandList);

	void Compile();

private:
	static void ExtendDynamicCompilerSection(FToolMenuSection& InSection);

	FSlateIcon GetCompileIcon() const;

	TObjectPtr<UDataLinkGraphAssetEditor> AssetEditor;

	UE::DataLink::EGraphCompileStatus LastCompiledStatus;
};

