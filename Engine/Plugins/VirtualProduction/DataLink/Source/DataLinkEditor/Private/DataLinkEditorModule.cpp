// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkEditorModule.h"
#include "Compiler/DataLinkGraphCompiler.h"
#include "DataLinkDelegates.h"
#include "DataLinkEditorLog.h"
#include "DataLinkGraphCommands.h"
#include "DataLinkInstance.h"
#include "DetailsView/DataLinkInputDataCustomization.h"
#include "DetailsView/DataLinkInstanceCustomization.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

DEFINE_LOG_CATEGORY(LogDataLinkEditor);

IMPLEMENT_MODULE(FDataLinkEditorModule, DataLinkEditor)

void FDataLinkEditorModule::StartupModule()
{
	FDataLinkGraphCommands::Register();
	RegisterDetailCustomizations();

	UE::DataLink::OnRequestCompilation.BindLambda(
		[](UDataLinkGraph* InGraphToCompile)->UE::DataLink::EGraphCompileStatus
		{
			FDataLinkGraphCompiler GraphCompiler(InGraphToCompile);
			return GraphCompiler.Compile();
		});
}

void FDataLinkEditorModule::ShutdownModule()
{
	FDataLinkGraphCommands::Unregister();
	UnregisterDetailCustomizations();

	UE::DataLink::OnRequestCompilation.Unbind();
}

void FDataLinkEditorModule::RegisterDetailCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FDataLinkInputData::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataLinkInputDataCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FDataLinkInstance::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataLinkInstanceCustomization::MakeInstance, /*bGenerateHeader*/true));
}

void FDataLinkEditorModule::UnregisterDetailCustomizations()
{
	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (FName CustomizedType : CustomizedTypes)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(CustomizedType);
		}
		CustomizedTypes.Reset();
	}
}
