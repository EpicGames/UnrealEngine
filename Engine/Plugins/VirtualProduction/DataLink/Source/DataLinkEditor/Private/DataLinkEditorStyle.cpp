// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkEditorStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"

FDataLinkEditorStyle& FDataLinkEditorStyle::Get()
{
	static FDataLinkEditorStyle Instance;
	return Instance;
}

FDataLinkEditorStyle::FDataLinkEditorStyle()
	: FSlateStyleSet(TEXT("DataLinkEditor"))
{
	ParentStyleName = FAppStyle::Get().GetStyleSetName();

	const FVector2f Icon20(20.f);

	ContentRootDir = FPaths::EngineContentDir() / TEXT("Editor/Slate");
	CoreContentRootDir = FPaths::EngineContentDir() / TEXT("Slate");

	// Editor Commands
	Set("DataLinkGraph.RunPreview", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon20));
	Set("DataLinkGraph.StopPreview", new CORE_IMAGE_BRUSH_SVG("Starship/Common/stop", Icon20));
	Set("DataLinkGraph.ClearPreviewOutput", new IMAGE_BRUSH_SVG("Starship/Common/ResetToDefault", Icon20));
	Set("DataLinkGraph.ClearPreviewCache", new IMAGE_BRUSH("Icons/GeneralTools/Delete_40x", Icon20));

	Set("CompileStatus.Background.Unknown", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", Icon20, FStyleColors::AccentYellow));
	Set("CompileStatus.Background.Warning", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", Icon20, FStyleColors::Warning));
	Set("CompileStatus.Background.Error"  , new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", Icon20, FStyleColors::Error));
	Set("CompileStatus.Background.Good"   , new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", Icon20, FStyleColors::AccentGreen));

	Set("CompileStatus.Overlay.Unknown", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Unknown_Badge", Icon20, FStyleColors::AccentYellow));
	Set("CompileStatus.Overlay.Warning", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Warning_Badge", Icon20, FStyleColors::Warning));
	Set("CompileStatus.Overlay.Error"  , new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Fail_Badge"   , Icon20, FStyleColors::Error));
	Set("CompileStatus.Overlay.Good"   , new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Good_Badge"   , Icon20, FStyleColors::AccentGreen));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FDataLinkEditorStyle::~FDataLinkEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
