// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FDataLinkEditorStyle final : public FSlateStyleSet
{
public:
	static FDataLinkEditorStyle& Get();

	FDataLinkEditorStyle();

	virtual ~FDataLinkEditorStyle() override;
};
