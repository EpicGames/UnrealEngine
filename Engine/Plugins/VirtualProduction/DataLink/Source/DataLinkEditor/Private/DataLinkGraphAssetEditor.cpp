// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphAssetEditor.h"
#include "DataLinkEdGraph.h"
#include "DataLinkGraph.h"
#include "DataLinkGraphAssetToolkit.h"

void UDataLinkGraphAssetEditor::Initialize(UDataLinkGraph* InDataLinkGraph)
{
	DataLinkGraph = InDataLinkGraph;

	if (UDataLinkEdGraph* Graph = GetDataLinkEdGraph())
	{
		Graph->InitializeNodes();
	}

	Super::Initialize();
}

TSharedPtr<FDataLinkGraphAssetToolkit> UDataLinkGraphAssetEditor::GetToolkit() const
{
	if (ToolkitInstance)
	{
		return StaticCastSharedRef<FDataLinkGraphAssetToolkit>(ToolkitInstance->AsShared());
	}
	return nullptr;
}

TSharedPtr<FUICommandList> UDataLinkGraphAssetEditor::GetToolkitCommands() const
{
	if (ToolkitInstance)
	{
		return ToolkitInstance->GetToolkitCommands();
	}
	return nullptr;
}

UDataLinkGraph* UDataLinkGraphAssetEditor::GetDataLinkGraph() const
{
	return DataLinkGraph;
}

UDataLinkEdGraph* UDataLinkGraphAssetEditor::GetDataLinkEdGraph() const
{
	if (DataLinkGraph)
	{
		return Cast<UDataLinkEdGraph>(DataLinkGraph->GetEdGraph());
	}
	return nullptr;
}

void UDataLinkGraphAssetEditor::GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit)
{
	Super::GetObjectsToEdit(InObjectsToEdit);
	InObjectsToEdit.Add(GetDataLinkGraph());
}

TSharedPtr<FBaseAssetToolkit> UDataLinkGraphAssetEditor::CreateToolkit()
{
	return MakeShared<FDataLinkGraphAssetToolkit>(this);
}
