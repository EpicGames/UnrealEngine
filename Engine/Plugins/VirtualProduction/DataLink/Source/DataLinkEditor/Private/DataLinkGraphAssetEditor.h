// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UAssetEditor.h"
#include "DataLinkGraphAssetEditor.generated.h"

class FDataLinkGraphAssetToolkit;
class UDataLinkEdGraph;
class UDataLinkGraph;

UCLASS(Transient)
class UDataLinkGraphAssetEditor : public UAssetEditor
{
	GENERATED_BODY()

public:
	void Initialize(UDataLinkGraph* InDataLinkGraph);

	TSharedPtr<FDataLinkGraphAssetToolkit> GetToolkit() const;

	TSharedPtr<FUICommandList> GetToolkitCommands() const;

	UDataLinkGraph* GetDataLinkGraph() const;

	UDataLinkEdGraph* GetDataLinkEdGraph() const;

protected:
	//~ Begin UAssetEditor
	virtual void GetObjectsToEdit(TArray<UObject*>& InObjectsToEdit) override;
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;
	//~ End UAssetEditor

private:
	UPROPERTY()
	TObjectPtr<UDataLinkGraph> DataLinkGraph;
};
