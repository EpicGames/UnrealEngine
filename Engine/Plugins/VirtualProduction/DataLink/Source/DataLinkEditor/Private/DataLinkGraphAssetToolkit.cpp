// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphAssetToolkit.h"
#include "Compiler/DataLinkGraphCompilerTool.h"
#include "DataLinkEdGraph.h"
#include "DataLinkGraph.h"
#include "DataLinkGraphAssetEditor.h"
#include "DataLinkGraphEditorMenuContext.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Graph/DataLinkGraphEditorTool.h"
#include "Modules/ModuleManager.h"
#include "Preview/DataLinkPreviewTool.h"
#include "PropertyEditorModule.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "DataLinkGraphAssetToolkit"

FDataLinkGraphAssetToolkit::FDataLinkGraphAssetToolkit(UDataLinkGraphAssetEditor* InAssetEditor)
	: FBaseAssetToolkit(InAssetEditor)
	, AssetEditor(InAssetEditor)
	, GraphTool(MakeShared<FDataLinkGraphEditorTool>(InAssetEditor))
	, CompilerTool(MakeShared<FDataLinkGraphCompilerTool>(InAssetEditor))
	, PreviewTool(MakeShared<FDataLinkPreviewTool>(InAssetEditor))
{
	LayoutAppendix = TEXT("DataLinkGraphAssetEditor");

	StandaloneDefaultLayout = FTabManager::NewLayout(FName(TEXT("Standalone_Layout_V_1_0_") + LayoutAppendix))
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.7f)
				->AddTab(FDataLinkGraphEditorTool::GraphEditorTabID, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.3f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(DetailsTabID, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(FDataLinkPreviewTool::PreviewTabID, ETabState::OpenedTab)
				)
			)
		);
}

TSharedPtr<IDetailsView> FDataLinkGraphAssetToolkit::GetDetailsView() const
{
	return DetailsView;
}

void FDataLinkGraphAssetToolkit::RegisterToolbar()
{
	FName ParentName;
	const FName MenuName = GetToolMenuToolbarName(ParentName);

	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus || ToolMenus->IsMenuRegistered(MenuName))
	{
		return;
	}

	FToolMenuOwnerScoped ToolMenuOwnerScope(this);

	UToolMenu* const ToolbarMenu = ToolMenus->RegisterMenu(MenuName, ParentName, EMultiBoxType::ToolBar);
	check(ToolbarMenu);

	CompilerTool->ExtendMenu(ToolbarMenu);
}

void FDataLinkGraphAssetToolkit::InitToolMenuContext(FToolMenuContext& InMenuContext)
{
	FBaseAssetToolkit::InitToolMenuContext(InMenuContext);

	UDataLinkGraphEditorMenuContext* Context = NewObject<UDataLinkGraphEditorMenuContext>();
	Context->ToolkitWeak = SharedThis(this);
	InMenuContext.AddObject(Context);
}

void FDataLinkGraphAssetToolkit::CreateWidgets()
{
	// Skip FBaseAssetToolkit to avoid creating a Viewport Tab/Client/etc
	RegisterToolbar();
	CreateEditorModeManager();

	LayoutExtender = MakeShared<FLayoutExtender>();

	GraphTool->CreateWidgets();
	PreviewTool->CreateWidgets();

	DetailsView = CreateDetailsView();
}

TSharedRef<IDetailsView> FDataLinkGraphAssetToolkit::CreateDetailsView()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;

	return PropertyEditorModule.CreateDetailView(DetailsViewArgs);
}

void FDataLinkGraphAssetToolkit::PostInitAssetEditor()
{
	FBaseAssetToolkit::PostInitAssetEditor();

	if (UDataLinkEdGraph* EdGraph = AssetEditor->GetDataLinkEdGraph())
	{
		EdGraph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateSP(this, &FDataLinkGraphAssetToolkit::OnGraphChanged));
	}
}

void FDataLinkGraphAssetToolkit::MapToolkitCommands()
{
	FBaseAssetToolkit::MapToolkitCommands();

	GraphTool->BindCommands(ToolkitCommands);
	CompilerTool->BindCommands(ToolkitCommands);
	PreviewTool->BindCommands(ToolkitCommands);
}

FName FDataLinkGraphAssetToolkit::GetToolkitFName() const
{
	return TEXT("FDataLinkGraphAssetToolkit");
}

void FDataLinkGraphAssetToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// Skip FBaseAssetToolkit to not register a viewport tab.
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	GraphTool->RegisterTabSpawners(InTabManager, AssetEditorTabsCategory);
	PreviewTool->RegisterTabSpawners(InTabManager, AssetEditorTabsCategory);

	InTabManager->RegisterTabSpawner(DetailsTabID, FOnSpawnTab::CreateSP(this, &FDataLinkGraphAssetToolkit::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("Details", "Details"))
		.SetGroup(AssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FDataLinkGraphAssetToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	GraphTool->UnregisterTabSpawners(InTabManager);
	PreviewTool->UnregisterTabSpawners(InTabManager);
	InTabManager->UnregisterTabSpawner(DetailsTabID);
}

void FDataLinkGraphAssetToolkit::OnGraphChanged(const FEdGraphEditAction& InAction)
{
	if (UDataLinkEdGraph* EdGraph = AssetEditor->GetDataLinkEdGraph())
	{
		EdGraph->DirtyGraph();
	}
}

#undef LOCTEXT_NAMESPACE
