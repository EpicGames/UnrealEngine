// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/BaseAssetToolkit.h"

class FDataLinkGraphCompilerTool;
class FDataLinkGraphEditorTool;
class FDataLinkPreviewTool;
class SGraphEditor;
class UDataLinkGraphAssetEditor;
struct FEdGraphEditAction;

class FDataLinkGraphAssetToolkit : public FBaseAssetToolkit
{
public:
	explicit FDataLinkGraphAssetToolkit(UDataLinkGraphAssetEditor* InAssetEditor);

	TSharedPtr<IDetailsView> GetDetailsView() const;

	//~ Begin FBaseAssetToolkit
	virtual void RegisterToolbar() override;
	virtual void InitToolMenuContext(FToolMenuContext& InMenuContext) override;
	virtual void CreateWidgets() override;
	virtual TSharedRef<IDetailsView> CreateDetailsView() override;
	virtual void PostInitAssetEditor() override;
	//~ End FBaseAssetToolkit

	//~ Begin FAssetEditorToolkit
	virtual void MapToolkitCommands() override;
	//~ End FAssetEditorToolkit

	//~ Begin IToolkit
	virtual FName GetToolkitFName() const override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	//~ End IToolkit

	const FDataLinkGraphCompilerTool& GetCompilerTool() const
	{
		return CompilerTool.Get();
	}

	const FDataLinkPreviewTool& GetPreviewTool() const
	{
		return PreviewTool.Get();
	}

private:
	void OnGraphChanged(const FEdGraphEditAction& InAction);

	TObjectPtr<UDataLinkGraphAssetEditor> AssetEditor;

	TSharedRef<FDataLinkGraphEditorTool> GraphTool;

	TSharedRef<FDataLinkGraphCompilerTool> CompilerTool;

	TSharedRef<FDataLinkPreviewTool> PreviewTool;
};
