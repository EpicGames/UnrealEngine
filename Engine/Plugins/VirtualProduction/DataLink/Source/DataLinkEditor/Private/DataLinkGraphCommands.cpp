// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphCommands.h"
#include "DataLinkEditorStyle.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "DataLinkGraphCommands"

FDataLinkGraphCommands::FDataLinkGraphCommands()
	: TCommands(TEXT("DataLinkGraph")
	, LOCTEXT("DataLinkGraph", "Data Link Graph")
	, NAME_None
	, FDataLinkEditorStyle::Get().GetStyleSetName())
{
}

void FDataLinkGraphCommands::RegisterCommands()
{
	UI_COMMAND(Compile
		, "Compile"
		, "Compiles the data link graph"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(RunPreview
		, "Preview"
		, "Triggers execution of the data link graph with the given input data"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(StopPreview
		, "Stop"
		, "Stops the active preview execution"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ClearPreviewOutput
		, "Clear Output"
		, "Clears the output data from the last execution."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ClearPreviewCache
		, "Clear Cache"
		, "Clears the cached node data from previous executions."
		, EUserInterfaceActionType::Button
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
