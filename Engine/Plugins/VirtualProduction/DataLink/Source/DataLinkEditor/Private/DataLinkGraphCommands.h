// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FDataLinkGraphCommands : public TCommands<FDataLinkGraphCommands>
{
public:
	FDataLinkGraphCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> Compile;

	TSharedPtr<FUICommandInfo> RunPreview;

	TSharedPtr<FUICommandInfo> StopPreview;

	TSharedPtr<FUICommandInfo> ClearPreviewOutput;

	TSharedPtr<FUICommandInfo> ClearPreviewCache;
};
