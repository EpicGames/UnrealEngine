// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphEditorMenuContext.h"
#include "DataLinkGraphAssetToolkit.h"
#include "Preview/DataLinkPreviewData.h"
#include "Preview/DataLinkPreviewTool.h"
#include "StructUtils/StructView.h"

FConstStructView UDataLinkGraphEditorMenuContext::FindPreviewOutputData() const
{
	TSharedPtr<FDataLinkGraphAssetToolkit> Toolkit = ToolkitWeak.Pin();
	if (!Toolkit.IsValid())
	{
		return FConstStructView();
	}

	const UDataLinkPreviewData* const PreviewData = Toolkit->GetPreviewTool().GetPreviewData();
	if (!PreviewData)
	{
		return FConstStructView();
	}

	return PreviewData->OutputData;
}

FString UDataLinkGraphEditorMenuContext::GetAssetPath() const
{
	TSharedPtr<FDataLinkGraphAssetToolkit> Toolkit = ToolkitWeak.Pin();
	if (!Toolkit.IsValid())
	{
		return FString();
	}

	const TArray<UObject*>* Objects = Toolkit->GetObjectsCurrentlyBeingEdited();
	if (!Objects || Objects->IsEmpty())
	{
		return FString();
	}

	UObject* const Object = (*Objects)[0];
	if (!Object)
	{
		return FString();
	}

	if (const UPackage* Package = Object->GetPackage())
	{
		return FPaths::GetPath(Package->GetPathName());
	}

	return FString();
}
