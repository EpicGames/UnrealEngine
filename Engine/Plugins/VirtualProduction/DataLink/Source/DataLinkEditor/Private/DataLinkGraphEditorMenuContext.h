// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDataLinkEditorMenuContext.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "DataLinkGraphEditorMenuContext.generated.h"

class FDataLinkGraphAssetToolkit;

UCLASS()
class UDataLinkGraphEditorMenuContext : public UObject, public IDataLinkEditorMenuContext
{
	GENERATED_BODY()

public:
	//~ Begin IDataLinkEditorMenuContext
	virtual FConstStructView FindPreviewOutputData() const override;
	virtual FString GetAssetPath() const override;
	//~ End IDataLinkEditorMenuContext

	TWeakPtr<FDataLinkGraphAssetToolkit> ToolkitWeak;
};
