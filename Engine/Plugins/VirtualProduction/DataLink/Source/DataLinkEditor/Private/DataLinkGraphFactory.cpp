// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphFactory.h"
#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"
#include "DataLinkEdGraph.h"
#include "DataLinkEdGraphSchema.h"
#include "DataLinkGraph.h"
#include "Kismet2/BlueprintEditorUtils.h"

UDataLinkGraphFactory::UDataLinkGraphFactory()
{
	SupportedClass = UDataLinkGraph::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

FText UDataLinkGraphFactory::GetDisplayName() const
{
	return SupportedClass ? SupportedClass->GetDisplayNameText() : Super::GetDisplayName();
}

FString UDataLinkGraphFactory::GetDefaultNewAssetName() const
{
	// Short name removing "Motion Design" and "DataLink" prefix for new assets
	return TEXT("NewDataGraph");
}

uint32 UDataLinkGraphFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	return AssetTools.FindAdvancedAssetCategory("MotionDesignCategory");
}

UObject* UDataLinkGraphFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	if (!ensure(SupportedClass == InClass))
	{
		return nullptr;
	}

	UDataLinkGraph* DataLinkGraph = NewObject<UDataLinkGraph>(InParent, InName, InFlags);
	check(DataLinkGraph);

	DataLinkGraph->EdGraph = FBlueprintEditorUtils::CreateNewGraph(DataLinkGraph
		, NAME_None
		, UDataLinkEdGraph::StaticClass()
		, UDataLinkEdGraphSchema::StaticClass());
	check(DataLinkGraph->EdGraph);

	const UEdGraphSchema* Schema = DataLinkGraph->EdGraph->GetSchema();
	check(Schema);
	Schema->CreateDefaultNodesForGraph(*DataLinkGraph->EdGraph);

	return DataLinkGraph;
}
