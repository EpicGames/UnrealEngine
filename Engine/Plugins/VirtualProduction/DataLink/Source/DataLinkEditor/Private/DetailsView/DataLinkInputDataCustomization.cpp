// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkInputDataCustomization.h"
#include "DataLinkInstance.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "InstancedStructDetails.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::DataLink::Private
{
	TSharedRef<SWidget> MakeArrayIndexWidget(int32 InArrayIndex)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(INVTEXT("["))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(3, 0, 0, 0)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::AsNumber(InArrayIndex))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(3, 0, 0, 0)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(INVTEXT("]"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			];
	}
}

void FDataLinkInputDataCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	DisplayNameHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataLinkInputData, DisplayName)).ToSharedRef();
	DisplayNameHandle->MarkHiddenByCustomization();

	TSharedRef<SHorizontalBox> NameBox = SNew(SHorizontalBox);

	const int32 ArrayIndex = InPropertyHandle->GetArrayIndex();
	if (ArrayIndex != INDEX_NONE)
	{
		NameBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				UE::DataLink::Private::MakeArrayIndexWidget(ArrayIndex)
			];
	}

	NameBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(5, 0, 0, 0)
		[
			SNew(STextBlock)
			.Text(this, &FDataLinkInputDataCustomization::GetInputDisplayName)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	InHeaderRow
		.ShouldAutoExpand(true)
		.NameContent()
		[
			NameBox
		];
}

void FDataLinkInputDataCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const TSharedRef<IPropertyHandle> DataHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataLinkInputData, Data)).ToSharedRef();
	DataHandle->MarkHiddenByCustomization();
	InChildBuilder.AddCustomBuilder(MakeShared<FInstancedStructDataDetails>(DataHandle));
}

FText FDataLinkInputDataCustomization::GetInputDisplayName() const
{
	FText InputDisplayName;
	if (DisplayNameHandle.IsValid())
	{
		DisplayNameHandle->GetValueAsDisplayText(InputDisplayName);
	}
	return InputDisplayName;
}
