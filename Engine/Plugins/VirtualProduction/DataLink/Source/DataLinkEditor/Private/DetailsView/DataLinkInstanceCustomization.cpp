// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkInstanceCustomization.h"
#include "DataLinkGraph.h"
#include "DataLinkInstance.h"
#include "DataLinkPin.h"
#include "DataLinkPinReference.h"
#include "DataLinkUtils.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"

FDataLinkInstanceCustomization::FDataLinkInstanceCustomization(bool bInGenerateHeader)
	: bGenerateHeader(bInGenerateHeader)
{
}

FDataLinkInstanceCustomization::~FDataLinkInstanceCustomization()
{
	UDataLinkGraph::OnGraphCompiled().RemoveAll(this);
}

void FDataLinkInstanceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	UDataLinkGraph::OnGraphCompiled().AddSP(this, &FDataLinkInstanceCustomization::OnGraphCompiled);

	DataLinkGraphHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataLinkInstance, DataLinkGraph));
	check(DataLinkGraphHandle.IsValid());
	DataLinkGraphHandle->MarkHiddenByCustomization();
	DataLinkGraphHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDataLinkInstanceCustomization::OnGraphChanged));

	InputDataHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDataLinkInstance, InputData));
	check(InputDataHandle.IsValid());
	InputDataHandle->MarkHiddenByCustomization();

	if (bGenerateHeader)
	{
		InHeaderRow
			.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				DataLinkGraphHandle->CreatePropertyValueWidget()
			];
	}

	UpdateInputData();
}

void FDataLinkInstanceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	check(InputDataHandle.IsValid());

	UpdateInputData();

	TSharedRef<FDetailArrayBuilder> ArrayBuilder = MakeShared<FDetailArrayBuilder>(InputDataHandle.ToSharedRef()
		, /*bGenerateHeader*/false
		, /*bDisplayResetToDefault*/false
		, /*DisplayElementNum*/false);

	ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda(
		[](TSharedRef<IPropertyHandle> InElementHandle, int32 InIndex, IDetailChildrenBuilder& InChildBuilder)
		{
			InChildBuilder.AddProperty(InElementHandle);
		}));

	InChildBuilder.AddCustomBuilder(ArrayBuilder);
}

void FDataLinkInstanceCustomization::OnGraphCompiled(UDataLinkGraph* InDataLinkGraph)
{
	if (InDataLinkGraph == GetDataLinkGraph())
	{
		OnGraphChanged();
	}
}

UDataLinkGraph* FDataLinkInstanceCustomization::GetDataLinkGraph() const
{
	UObject* DataLink = nullptr;
	if (DataLinkGraphHandle.IsValid() && DataLinkGraphHandle->GetValue(DataLink) == FPropertyAccess::Success)
	{
		return Cast<UDataLinkGraph>(DataLink);
	}
	return nullptr;
}

TArray<FDataLinkInputData>* FDataLinkInstanceCustomization::GetInputData() const
{
	void* RawData = nullptr;
	if (InputDataHandle.IsValid() && InputDataHandle->GetValueData(RawData) == FPropertyAccess::Success)
	{
		return static_cast<TArray<FDataLinkInputData>*>(RawData);
	}
	return nullptr;
}

void FDataLinkInstanceCustomization::OnGraphChanged()
{
	if (!InputDataHandle.IsValid())
	{
		return;
	}

	InputDataHandle->NotifyPreChange();
	UpdateInputData();
	InputDataHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

void FDataLinkInstanceCustomization::UpdateInputData()
{
	if (TArray<FDataLinkInputData>* InputData = GetInputData())
	{
		UE::DataLink::SetInputData(GetDataLinkGraph(), *InputData);
	}
}
