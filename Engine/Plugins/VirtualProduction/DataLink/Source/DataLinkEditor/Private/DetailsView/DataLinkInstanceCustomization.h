// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class UDataLinkGraph;
struct FDataLinkInputData;
struct FInstancedStruct;

class FDataLinkInstanceCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance(bool bInGenerateHeader)
	{
		return MakeShared<FDataLinkInstanceCustomization>(bInGenerateHeader);
	}

	explicit FDataLinkInstanceCustomization(bool bInGenerateHeader);

	virtual ~FDataLinkInstanceCustomization() override;

protected:
	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

	void OnGraphCompiled(UDataLinkGraph* InDataLinkGraph);

	UDataLinkGraph* GetDataLinkGraph() const;

	TArray<FDataLinkInputData>* GetInputData() const;

	void OnGraphChanged();
	void UpdateInputData();

	TSharedPtr<IPropertyHandle> DataLinkGraphHandle;
	TSharedPtr<IPropertyHandle> InputDataHandle;

	bool bGenerateHeader;
};
