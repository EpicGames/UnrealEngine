// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkGraphEditorTool.h"
#include "BlueprintActionDatabase.h"
#include "Customizations/MathStructProxyCustomizations.h"
#include "DataLinkEdGraph.h"
#include "DataLinkGraphAssetEditor.h"
#include "DataLinkGraphAssetToolkit.h"
#include "EdGraphUtilities.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Docking/TabManager.h"
#include "GraphEditor.h"
#include "HAL/PlatformApplicationMisc.h"
#include "SNodePanel.h"
#include "ScopedTransaction.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "DataLinkGraphEditorToolkit"

const FLazyName FDataLinkGraphEditorTool::GraphEditorTabID = TEXT("DataLinkGraphAssetToolkit_Graph");

FDataLinkGraphEditorTool::FDataLinkGraphEditorTool(UDataLinkGraphAssetEditor* InAssetEditor)
	: AssetEditor(InAssetEditor)
	, GraphEditorCommands(MakeShared<FUICommandList>())
{
}

void FDataLinkGraphEditorTool::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	GraphEditorCommands->Append(InCommandList);

	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	GraphEditorCommands->MapAction(GenericCommands.SelectAll
		, FExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::SelectAllNodes)
	);

	GraphEditorCommands->MapAction(GenericCommands.Delete
		, FExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::DeleteSelectedNodes)
		, FCanExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::CanDeleteSelectedNodes)
	);

	GraphEditorCommands->MapAction(GenericCommands.Copy
		, FExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::CopySelectedNodes)
		, FCanExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::CanCopySelectedNodes)
	);

	GraphEditorCommands->MapAction(GenericCommands.Cut
		, FExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::CutSelectedNodes)
		, FCanExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::CanCutSelectedNodes)
	);

	GraphEditorCommands->MapAction(GenericCommands.Paste
		, FExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::PasteNodes)
		, FCanExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::CanPasteNodes)
	);

	GraphEditorCommands->MapAction(GenericCommands.Duplicate
		, FExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::DuplicateSelectedNodes)
		, FCanExecuteAction::CreateSP(this, &FDataLinkGraphEditorTool::CanCopySelectedNodes)
	);
}

void FDataLinkGraphEditorTool::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<FWorkspaceItem>& InAssetEditorTabsCategory)
{
	InTabManager->RegisterTabSpawner(GraphEditorTabID, FOnSpawnTab::CreateSP(this, &FDataLinkGraphEditorTool::SpawnTab))
		.SetDisplayName(LOCTEXT("Graph", "Graph"))
		.SetGroup(InAssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));
}

void FDataLinkGraphEditorTool::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(GraphEditorTabID);
}

void FDataLinkGraphEditorTool::CreateWidgets()
{
	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("GraphCornerText", "DATA LINK");

	SGraphEditor::FGraphEditorEvents Events;
	Events.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FDataLinkGraphEditorTool::OnSelectedNodesChanged);

	GraphEditor = SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(true)
		.Appearance(AppearanceInfo)
		.GraphToEdit(AssetEditor->GetDataLinkEdGraph())
		.GraphEvents(Events)
		.AutoExpandActionMenu(true)
		.ShowGraphStateOverlay(false);
}

void FDataLinkGraphEditorTool::SelectAllNodes()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->SelectAllNodes();
	}
}

bool FDataLinkGraphEditorTool::CanDeleteSelectedNodes() const
{
	if (!GraphEditor.IsValid())
	{
		return false;
	}

	int32 DeletableNodes = 0;

	for (const UObject* SelectedNode : GraphEditor->GetSelectedNodes())
	{
		const UEdGraphNode* Node = Cast<UEdGraphNode>(SelectedNode);
		if (Node && Node->CanUserDeleteNode())
		{
			++DeletableNodes;
		}
	}

	return DeletableNodes > 0;
}

void FDataLinkGraphEditorTool::DeleteSelectedNodes()
{
	if (!GraphEditor.IsValid())
	{
		return;
	}

	UEdGraph* EdGraph = AssetEditor->GetDataLinkEdGraph();
	if (!EdGraph)
	{
		return;
	}

	// Gather nodes to delete
	TArray<UEdGraphNode*> NodesToDelete;
	{
		const TSet<UObject*>& SelectedNodes = GraphEditor->GetSelectedNodes();
		NodesToDelete.Reserve(SelectedNodes.Num());

		for (UObject* SelectedNode : SelectedNodes)
		{
			UEdGraphNode* Node = Cast<UEdGraphNode>(SelectedNode);
			if (Node && Node->CanUserDeleteNode())
			{
				NodesToDelete.Add(Node);
			}
		}
	}

	if (NodesToDelete.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("DeleteSelectedNodes", "Delete Selected Nodes"));
	EdGraph->Modify();

	for (UEdGraphNode* Node : NodesToDelete)
	{
		EdGraph->RemoveNode(Node);
	}
}

bool FDataLinkGraphEditorTool::CanCopySelectedNodes() const
{
	return CanDuplicateSelectedNodes();
}

void FDataLinkGraphEditorTool::CopySelectedNodes()
{
	CopySelectedNodesInternal();
}

bool FDataLinkGraphEditorTool::CanCutSelectedNodes() const
{
	return CanCopySelectedNodes() && CanDeleteSelectedNodes();
}

void FDataLinkGraphEditorTool::CutSelectedNodes()
{
	UEdGraph* EdGraph = AssetEditor->GetDataLinkEdGraph();
	if (!EdGraph)
	{
		return;
	}

	TSet<UObject*> CopiedNodes = CopySelectedNodesInternal();
	if (CopiedNodes.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("CutSelectedNodes", "Cut Selected Nodes"));
	EdGraph->Modify();

	for (UObject* Node : CopiedNodes)
	{
		EdGraph->RemoveNode(CastChecked<UEdGraphNode>(Node));
	}
}

bool FDataLinkGraphEditorTool::CanPasteNodes() const
{
	if (!GraphEditor.IsValid())
	{
		return false;
	}

	UEdGraph* EdGraph = AssetEditor->GetDataLinkEdGraph();
	if (!EdGraph)
	{
		return false;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	return FEdGraphUtilities::CanImportNodesFromText(EdGraph, ClipboardContent);
}

void FDataLinkGraphEditorTool::PasteNodes()
{
	if (!GraphEditor.IsValid())
	{
		return;
	}

	UEdGraph* EdGraph = AssetEditor->GetDataLinkEdGraph();
	if (!EdGraph)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("PasteNodes", "Paste Nodes"));
	EdGraph->Modify();

	// Clear the selection set (newly pasted stuff will be selected)
	GraphEditor->ClearSelectionSet();

	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Import the nodes
	TSet<UEdGraphNode*> PastedNodes;
	FEdGraphUtilities::ImportNodesFromText(EdGraph, TextToImport, /*out*/ PastedNodes);

	if (PastedNodes.IsEmpty())
	{
		return;
	}

	// Average position of nodes so we can move them while still maintaining relative distances to each other
	FVector2f AverageNodePosition = FVector2f::ZeroVector;
	{
		for (UEdGraphNode* PastedNode : PastedNodes)
		{
			AverageNodePosition.X += PastedNode->NodePosX;
			AverageNodePosition.Y += PastedNode->NodePosY;
		}
		AverageNodePosition /= PastedNodes.Num();
	}

	const FVector2f PasteLocation = GraphEditor->GetPasteLocation2f();
	const uint32 SnapGridSize = SNodePanel::GetSnapGridSize();

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		// Select the newly pasted stuff
		GraphEditor->SetNodeSelection(PastedNode, /*bSelect*/true);

		PastedNode->NodePosX = (PastedNode->NodePosX - AverageNodePosition.X) + PasteLocation.X;
		PastedNode->NodePosY = (PastedNode->NodePosY - AverageNodePosition.Y) + PasteLocation.Y;

		PastedNode->SnapToGrid(SnapGridSize);

		// Give new node a different Guid from the old one
		PastedNode->CreateNewGuid();
	}

	GraphEditor->NotifyGraphChanged();
}

bool FDataLinkGraphEditorTool::CanDuplicateSelectedNodes() const
{
	if (!GraphEditor.IsValid())
	{
		return false;
	}

	int32 CanDuplicateCount = 0;

	for (const UObject* SelectedNode : GraphEditor->GetSelectedNodes())
	{
		const UEdGraphNode* Node = Cast<UEdGraphNode>(SelectedNode);
		if (Node && Node->CanDuplicateNode())
		{
			++CanDuplicateCount;
		}
	}

	return CanDuplicateCount > 0;
}

void FDataLinkGraphEditorTool::DuplicateSelectedNodes()
{
	CopySelectedNodes();
	PasteNodes();
}

TSet<UObject*> FDataLinkGraphEditorTool::CopySelectedNodesInternal()
{
	if (!GraphEditor.IsValid())
	{
		return {};
	}

	// Gather selected nodes
	TSet<UObject*> NodesToCopy = GraphEditor->GetSelectedNodes();

	// Notify nodes of copying, removing those that can't copy from the set
	for (TSet<UObject*>::TIterator Iter(NodesToCopy); Iter; ++Iter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*Iter);
		if (Node && Node->CanDuplicateNode())
		{
			Node->PrepareForCopying();
		}
		else
		{
			Iter.RemoveCurrent();
		}
	}

	FString ExportedText;
	FEdGraphUtilities::ExportNodesToText(NodesToCopy, ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

	return NodesToCopy;
}

TSharedRef<SDockTab> FDataLinkGraphEditorTool::SpawnTab(const FSpawnTabArgs& InTabArgs) const
{
	return SNew(SDockTab)
		.Label(LOCTEXT("GraphTitle", "Graph"))
		[
			GraphEditor.ToSharedRef()
		];
}

void FDataLinkGraphEditorTool::OnSelectedNodesChanged(const TSet<UObject*>& InSelectionSet)
{
	if (TSharedPtr<FDataLinkGraphAssetToolkit> Toolkit = AssetEditor->GetToolkit())
	{
		if (TSharedPtr<IDetailsView> DetailsView = Toolkit->GetDetailsView())
		{
			DetailsView->SetObjects(InSelectionSet.Array());
		}
	}
}

#undef LOCTEXT_NAMESPACE
