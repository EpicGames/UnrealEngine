// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"

class FSpawnTabArgs;
class FTabManager;
class FUICommandList;
class FWorkspaceItem;
class SDockTab;
class SGraphEditor;
class UDataLinkGraphAssetEditor;
class UObject;

class FDataLinkGraphEditorTool : public TSharedFromThis<FDataLinkGraphEditorTool>
{
public:
	static const FLazyName GraphEditorTabID;

	explicit FDataLinkGraphEditorTool(UDataLinkGraphAssetEditor* InAssetEditor);

	void BindCommands(const TSharedRef<FUICommandList>& InCommandList);

	void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<FWorkspaceItem>& InAssetEditorTabsCategory);
	void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager);

	void CreateWidgets();

protected:
	void SelectAllNodes();

	bool CanDeleteSelectedNodes() const;
	void DeleteSelectedNodes();

	bool CanCopySelectedNodes() const;
	void CopySelectedNodes();

	bool CanCutSelectedNodes() const;
	void CutSelectedNodes();

	bool CanPasteNodes() const;
	void PasteNodes();

	bool CanDuplicateSelectedNodes() const;
	void DuplicateSelectedNodes();

private:
	TSet<UObject*> CopySelectedNodesInternal();

	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& InTabArgs) const;

	void OnSelectedNodesChanged(const TSet<UObject*>& InSelectionSet);

	TObjectPtr<UDataLinkGraphAssetEditor> AssetEditor;

	TSharedRef<FUICommandList> GraphEditorCommands;

	TSharedPtr<SGraphEditor> GraphEditor;
};
