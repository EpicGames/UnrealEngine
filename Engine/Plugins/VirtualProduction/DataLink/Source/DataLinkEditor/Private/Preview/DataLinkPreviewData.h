// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkInstance.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/Object.h"
#include "DataLinkPreviewData.generated.h"

UCLASS()
class UDataLinkPreviewData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category="Input Data")
	FDataLinkInstance DataLinkInstance;

	UPROPERTY(VisibleAnywhere, Category="Output Data", meta=(StructTypeConst))
	FInstancedStruct OutputData;
};
