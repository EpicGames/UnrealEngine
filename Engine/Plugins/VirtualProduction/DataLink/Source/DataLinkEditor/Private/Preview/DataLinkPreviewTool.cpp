// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkPreviewTool.h"
#include "DataLinkEditorLog.h"
#include "DataLinkEditorNames.h"
#include "DataLinkEditorStyle.h"
#include "DataLinkEnums.h"
#include "DataLinkExecutor.h"
#include "DataLinkExecutorArguments.h"
#include "DataLinkGraphAssetEditor.h"
#include "DataLinkGraphAssetToolkit.h"
#include "DataLinkGraphCommands.h"
#include "DataLinkGraphEditorMenuContext.h"
#include "DataLinkInstance.h"
#include "DataLinkPreviewData.h"
#include "DetailsView/DataLinkInstanceCustomization.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "DataLinkPreviewTool"

const FLazyName FDataLinkPreviewTool::PreviewTabID = TEXT("DataLinkGraphAssetToolkit_Preview");

FDataLinkPreviewTool::FDataLinkPreviewTool(UDataLinkGraphAssetEditor* InAssetEditor)
	: AssetEditor(InAssetEditor)
{
}

void FDataLinkPreviewTool::Initialize()
{
	PreviewData = NewObject<UDataLinkPreviewData>(GetTransientPackage());
	PreviewData->DataLinkInstance.DataLinkGraph = AssetEditor->GetDataLinkGraph();
}

void FDataLinkPreviewTool::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	const FDataLinkGraphCommands& GraphCommands = FDataLinkGraphCommands::Get();

	InCommandList->MapAction(GraphCommands.RunPreview
		, FExecuteAction::CreateSP(this, &FDataLinkPreviewTool::RunPreview)
		, FCanExecuteAction::CreateSP(this, &FDataLinkPreviewTool::CanRunPreview));

	InCommandList->MapAction(GraphCommands.StopPreview
		, FExecuteAction::CreateSP(this, &FDataLinkPreviewTool::StopPreview)
		, FCanExecuteAction::CreateSP(this, &FDataLinkPreviewTool::CanStopPreview));

	InCommandList->MapAction(GraphCommands.ClearPreviewOutput
		, FExecuteAction::CreateSP(this, &FDataLinkPreviewTool::ClearOutput)
		, FCanExecuteAction::CreateSP(this, &FDataLinkPreviewTool::CanClearOutput));

	InCommandList->MapAction(GraphCommands.ClearPreviewCache
		, FExecuteAction::CreateSP(this, &FDataLinkPreviewTool::ClearCache)
		, FCanExecuteAction::CreateSP(this, &FDataLinkPreviewTool::CanClearCache));
}

void FDataLinkPreviewTool::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<FWorkspaceItem>& InAssetEditorTabsCategory)
{
	InTabManager->RegisterTabSpawner(PreviewTabID, FOnSpawnTab::CreateSP(this, &FDataLinkPreviewTool::SpawnTab))
		.SetDisplayName(LOCTEXT("Preview", "Preview"))
		.SetGroup(InAssetEditorTabsCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FDataLinkEditorStyle::Get().GetStyleSetName(), "DataLinkGraph.Preview"));
}

void FDataLinkPreviewTool::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(PreviewTabID);
}

void FDataLinkPreviewTool::CreateWidgets()
{
	Initialize();
	CreateDetailsView();
	RegisterToolbar();
}

FString FDataLinkPreviewTool::GetReferencerName() const
{
	return TEXT("DataLinkPreviewTool");
}

void FDataLinkPreviewTool::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(PreviewData);
}

bool FDataLinkPreviewTool::CanRunPreview() const
{
	// Allow execution if there's not already one taking place
	return !Executor.IsValid() || !Executor->IsRunning();
}

void FDataLinkPreviewTool::RunPreview()
{
	if (!CanRunPreview())
	{
		UE_LOGF(LogDataLinkEditor, Error, "[%ls] Data Link execution is in progress!", Executor->GetContextName().GetData());
		return;
	}

	if (!Sink.IsValid())
	{
		Sink = MakeShared<FDataLinkSink>();
	}

	Executor = FDataLinkExecutor::Create(FDataLinkExecutorArguments(PreviewData->DataLinkInstance)
#if WITH_DATALINK_CONTEXT
		.SetContextName(TEXT("Graph Editor Preview"))
#endif
		.SetSink(Sink)
		.SetOnOutputData(FOnDataLinkOutputData::CreateSP(this, &FDataLinkPreviewTool::OnPreviewOutputData)));

	Executor->Run();
}

bool FDataLinkPreviewTool::CanStopPreview() const
{
	// Can only stop preview if there's an active executor
	return Executor.IsValid() && Executor->IsRunning();
}

void FDataLinkPreviewTool::StopPreview()
{
	if (CanStopPreview())
	{
		Executor->Stop();
		Executor.Reset();
	}
}

bool FDataLinkPreviewTool::CanClearOutput() const
{
	return PreviewData && PreviewData->OutputData.IsValid();
}

void FDataLinkPreviewTool::ClearOutput()
{
	if (PreviewData)
	{
		PreviewData->OutputData.Reset();
	}
}

bool FDataLinkPreviewTool::CanClearCache() const
{
	// Even though Executor saves a shared ref of the Sink and will not affect if this Sink ref is cleared during execution,
	// it might set wrong expectations to the user... e.g. user might think that clearing cache during execution might have an effect.
	// So only allow clearing cache if there's a valid sink and no execution is taking place.
	return Sink.IsValid() && !Executor.IsValid();
}

void FDataLinkPreviewTool::ClearCache()
{
	Sink.Reset();
}

void FDataLinkPreviewTool::OnPreviewOutputData(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView)
{
	if (!InOutputDataView.IsValid())
	{
		PreviewData->OutputData.Reset();
		return;
	}

	const UScriptStruct* OutputDataStruct = InOutputDataView.GetScriptStruct();
	if (PreviewData->OutputData.GetScriptStruct() != OutputDataStruct)
	{
		PreviewData->OutputData.InitializeAs(OutputDataStruct);
	}

	OutputDataStruct->CopyScriptStruct(PreviewData->OutputData.GetMutableMemory(), InOutputDataView.GetMemory());
	PreviewDetails->ForceRefresh();
}

void FDataLinkPreviewTool::RegisterToolbar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus || ToolMenus->IsMenuRegistered(UE::DataLinkEditor::PreviewToolbarName))
	{
		return;
	}

	UToolMenu* const ToolbarMenu = ToolMenus->RegisterMenu(UE::DataLinkEditor::PreviewToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
	if (!ToolbarMenu)
	{
		return;
	}

	const FDataLinkGraphCommands& GraphCommands = FDataLinkGraphCommands::Get();

	FToolMenuSection& PreviewSection = ToolbarMenu->FindOrAddSection(UE::DataLinkEditor::PreviewSectionName);
	PreviewSection.AddEntry(FToolMenuEntry::InitToolBarButton(GraphCommands.RunPreview));
	PreviewSection.AddEntry(FToolMenuEntry::InitToolBarButton(GraphCommands.StopPreview));
	PreviewSection.AddEntry(FToolMenuEntry::InitToolBarButton(GraphCommands.ClearPreviewOutput));
	PreviewSection.AddEntry(FToolMenuEntry::InitToolBarButton(GraphCommands.ClearPreviewCache));
}

void FDataLinkPreviewTool::CreateDetailsView()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;

	PreviewDetails = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	PreviewDetails->RegisterInstancedCustomPropertyTypeLayout(FDataLinkInstance::StaticStruct()->GetFName()
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataLinkInstanceCustomization::MakeInstance, /*bGenerateHeader*/false));

	PreviewDetails->SetObject(PreviewData);
}

TSharedRef<SWidget> FDataLinkPreviewTool::CreateContentWidget() const
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			CreateToolbar()
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			PreviewDetails.ToSharedRef()
		];
}

TSharedRef<SWidget> FDataLinkPreviewTool::CreateToolbar() const
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(AssetEditor && ToolMenus);

	FToolMenuContext Context(AssetEditor->GetToolkitCommands());

	UDataLinkGraphEditorMenuContext* MenuContext = NewObject<UDataLinkGraphEditorMenuContext>();
	MenuContext->ToolkitWeak = AssetEditor->GetToolkit();
	Context.AddObject(MenuContext);

	return ToolMenus->GenerateWidget(UE::DataLinkEditor::PreviewToolbarName, Context);
}

TSharedRef<SDockTab> FDataLinkPreviewTool::SpawnTab(const FSpawnTabArgs& InTabArgs) const
{
	return SNew(SDockTab)
		.Label(LOCTEXT("PreviewTitle", "Preview"))
		[
			CreateContentWidget()
		];
}

#undef LOCTEXT_NAMESPACE
