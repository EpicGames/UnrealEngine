// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"

class FDataLinkExecutor;
class FSpawnTabArgs;
class FTabManager;
class FUICommandList;
class FWorkspaceItem;
class IDetailsView;
class SDockTab;
class SWidget;
class UDataLinkGraphAssetEditor;
class UDataLinkPreviewData;
enum class EDataLinkExecutionResult : uint8;
struct FConstStructView;
struct FDataLinkSink;

/** Handles executing a preview of the Data Link Graph with input data from within the Editor itself */
class FDataLinkPreviewTool : public FGCObject, public TSharedFromThis<FDataLinkPreviewTool>
{
public:
	static const FLazyName PreviewTabID;

	explicit FDataLinkPreviewTool(UDataLinkGraphAssetEditor* InAssetEditor);

	void Initialize();

	void BindCommands(const TSharedRef<FUICommandList>& InCommandList);

	void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<FWorkspaceItem>& InAssetEditorTabsCategory);

	void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager);

	void CreateWidgets();

	const UDataLinkPreviewData* GetPreviewData() const
	{
		return PreviewData;
	}

protected:
	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

private:
	bool CanRunPreview() const;
	void RunPreview();

	bool CanStopPreview() const;
	void StopPreview();

	bool CanClearOutput() const;
	void ClearOutput();

	bool CanClearCache() const;
	void ClearCache();

	void OnPreviewOutputData(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView);

	void RegisterToolbar();

	void CreateDetailsView();

	TSharedRef<SWidget> CreateContentWidget() const;

	TSharedRef<SWidget> CreateToolbar() const;

	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& InTabArgs) const;

	TObjectPtr<UDataLinkGraphAssetEditor> AssetEditor;

	TSharedPtr<IDetailsView> PreviewDetails;

	/** Preview object holding the input and output data */
	TObjectPtr<UDataLinkPreviewData> PreviewData;

	/** Current Executor taking place */
	TSharedPtr<FDataLinkExecutor> Executor;

	/** Data Sink to use for Executor */
	TSharedPtr<FDataLinkSink> Sink;
};
