// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::DataLinkEditor
{
	const FLazyName PreviewToolbarName(TEXT("DataLinkGraphAssetToolkit_PreviewToolbar"));
	const FLazyName PreviewSectionName(TEXT("Preview"));
}
