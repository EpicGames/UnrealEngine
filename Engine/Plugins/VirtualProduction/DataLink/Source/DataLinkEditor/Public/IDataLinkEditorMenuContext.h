// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IDataLinkEditorMenuContext.generated.h"

struct FConstStructView;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UDataLinkEditorMenuContext : public UInterface
{
	GENERATED_BODY()
};

class IDataLinkEditorMenuContext
{
	GENERATED_BODY()

public:
	/** 
	 * Finds the latest preview output data. Could return invalid struct.
	 * @note Existing views can be invalidated at any time by the user so handle these views as short-lived!
	 */
	virtual FConstStructView FindPreviewOutputData() const = 0;

	/** Retrieves the asset path of the currently edited asset */
	virtual FString GetAssetPath() const = 0;
};
