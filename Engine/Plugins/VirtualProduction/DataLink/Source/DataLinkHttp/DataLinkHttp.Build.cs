// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataLinkHttp : ModuleRules
{
    public DataLinkHttp(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "DataLink",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "HTTP",
            }
        );
    }
}