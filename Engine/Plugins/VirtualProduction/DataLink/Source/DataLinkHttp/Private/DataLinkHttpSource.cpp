// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkHttpSource.h"
#include "DataLinkCoreTypes.h"
#include "DataLinkEnums.h"
#include "DataLinkExecutor.h"
#include "DataLinkHttpNames.h"
#include "DataLinkHttpSettings.h"
#include "DataLinkNames.h"
#include "DataLinkPinBuilder.h"
#include "DataLinkSink.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"

#define LOCTEXT_NAMESPACE "DataLinkHttpSource"

void UDataLinkHttpSource::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	Super::OnBuildPins(Inputs, Outputs);

	Inputs.Add(UE::DataLinkHttp::InputHttpSettings)
		.SetDisplayName(LOCTEXT("HttpSettingsDisplay", "Http Settings"))
		.SetStruct<FDataLinkHttpSettings>();

	Outputs.Add(UE::DataLink::OutputDefault)
		.SetDisplayName(LOCTEXT("OutputDisplay", "Response String"))
		.SetStruct<FDataLinkString>();
}

EDataLinkExecutionReply UDataLinkHttpSource::OnExecute(FDataLinkExecutor& InExecutor) const
{
	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	const FDataLinkInputDataViewer& InputDataViewer = NodeInstance.GetInputDataViewer();

	const FDataLinkHttpSettings& InputData = InputDataViewer.Get<FDataLinkHttpSettings>(UE::DataLinkHttp::InputHttpSettings);

	const TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetURL(InputData.URL);
	HttpRequest->SetVerb(InputData.Verb);

	for (const TPair<FString, FString>& HeaderPair : InputData.Headers)
	{
		HttpRequest->SetHeader(HeaderPair.Key, HeaderPair.Value);
	}

	HttpRequest->SetContentAsString(InputData.Body);
	HttpRequest->SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnGameThread);

	HttpRequest->OnProcessRequestComplete().BindWeakLambda(this,
		[this, ExecutorWeak = InExecutor.AsWeak()](FHttpRequestPtr InRequest, FHttpResponsePtr InResponse, bool bInProcessedSuccessfully)
		{
			// Delegate Thread Policy set to Complete on Game Thread
			check(IsInGameThread());

			TSharedPtr<FDataLinkExecutor> Executor = ExecutorWeak.Pin();
			if (!Executor.IsValid())
			{
				return;
			}

			if (bInProcessedSuccessfully)
			{
				const FDataLinkNodeInstance& NodeInstance = Executor->GetNodeInstance(this);

				const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

				FDataLinkString& OutputDataView = OutputDataViewer.Get<FDataLinkString>(UE::DataLink::OutputDefault);
				OutputDataView.Value = InResponse->GetContentAsString();

				Executor->Next(this);
			}
			else
			{
				Executor->Fail(this);
			}
		});

	// If process request fails, it will be handled within 'OnProcessRequestComplete' lambda above
	HttpRequest->ProcessRequest();
	return EDataLinkExecutionReply::Handled;
}

#undef LOCTEXT_NAMESPACE
