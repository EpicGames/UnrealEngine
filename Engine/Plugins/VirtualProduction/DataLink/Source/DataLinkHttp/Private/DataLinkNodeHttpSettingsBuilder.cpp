// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkNodeHttpSettingsBuilder.h"
#include "DataLinkCoreTypes.h"
#include "DataLinkExecutor.h"
#include "DataLinkHttpSettings.h"
#include "DataLinkNames.h"
#include "DataLinkNodeInstance.h"
#include "DataLinkPinBuilder.h"

#define LOCTEXT_NAMESPACE "DataLinkHttpSettingsBuilder"

void UDataLinkNodeHttpSettingsBuilder::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	FDataLinkStringBuilder(URLSegments, Tokens)
		.BuildInputPins(Inputs);

	Outputs.Add(UE::DataLink::OutputDefault)
		.SetDisplayName(LOCTEXT("HttpSettings", "Http Settings"))
		.SetStruct<FDataLinkHttpSettings>();
}

EDataLinkExecutionReply UDataLinkNodeHttpSettingsBuilder::OnExecute(FDataLinkExecutor& InExecutor) const
{
	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

	FDataLinkHttpSettings& HttpSettings = OutputDataViewer.Get<FDataLinkHttpSettings>(UE::DataLink::OutputDefault);

	const FDataLinkStringBuilder URLBuilder(URLSegments, Tokens);
	if (!URLBuilder.BuildString(NodeInstance.GetInputDataViewer(), HttpSettings.URL))
	{
		InExecutor.Fail(this);
		return EDataLinkExecutionReply::Handled;
	}

	HttpSettings.Verb = Verb;
	HttpSettings.Headers = Headers;
	HttpSettings.Body = Body;

	InExecutor.Next(this);
	return EDataLinkExecutionReply::Handled;
}

#if WITH_EDITOR
void UDataLinkNodeHttpSettingsBuilder::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UDataLinkNodeHttpSettingsBuilder, URLSegments))
	{
		FDataLinkStringBuilder::GatherTokens(URLSegments, Tokens);
	}
}
#endif

#undef LOCTEXT_NAMESPACE
