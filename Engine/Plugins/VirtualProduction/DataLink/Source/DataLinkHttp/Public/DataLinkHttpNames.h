// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::DataLinkHttp
{
	const FLazyName InputHttpSettings(TEXT("InputHttpSettings"));
}
