// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DataLinkHttpSettings.generated.h"

USTRUCT(BlueprintType, DisplayName="Data Link Http Settings")
struct FDataLinkHttpSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Http")
	FString URL;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Http")
	FString Verb = TEXT("GET");

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Http")
	TMap<FString, FString> Headers;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Http")
	FString Body;
};
