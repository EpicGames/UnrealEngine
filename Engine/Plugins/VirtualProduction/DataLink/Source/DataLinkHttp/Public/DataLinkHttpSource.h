// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "DataLinkHttpSource.generated.h"

UCLASS(MinimalAPI, DisplayName="Http Request", Category="Http")
class UDataLinkHttpSource : public UDataLinkNode
{
    GENERATED_BODY()

protected:
    //~ Begin UDataLinkNode
    DATALINKHTTP_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
    DATALINKHTTP_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
    //~ End UDataLinkNode
};
