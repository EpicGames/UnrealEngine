// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DataLinkNode.h"
#include "Nodes/String/DataLinkStringBuilder.h"
#include "DataLinkNodeHttpSettingsBuilder.generated.h"

/** Builder Node to help layer Http Settings with a URL Builder Interface */
UCLASS(MinimalAPI, DisplayName="Http Settings Builder", Category="Http")
class UDataLinkNodeHttpSettingsBuilder : public UDataLinkNode
{
	GENERATED_BODY()

protected:
	//~ Begin UDataLinkNode
	DATALINKHTTP_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
	DATALINKHTTP_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
	//~ End UDataLinkNode

	//~ Begin UObject
#if WITH_EDITOR
	DATALINKHTTP_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

private:
	/** Segments to build the URL from, where a token can be defined as {Token Name} in its own isolated entry */
	UPROPERTY(EditAnywhere, Category="URL", DisplayName="URL Segments", meta=(InvalidatesNode))
	TArray<FString> URLSegments;

	/** The tokens found within the Segment Array */
	UPROPERTY()
	TArray<FDataLinkStringBuilderToken> Tokens;

	UPROPERTY(EditAnywhere, Category="Settings", AdvancedDisplay)
	FString Verb = TEXT("GET");

	UPROPERTY(EditAnywhere, Category="Settings", AdvancedDisplay)
	TMap<FString, FString> Headers;

	UPROPERTY(EditAnywhere, Category="Settings", AdvancedDisplay)
	FString Body;
};
