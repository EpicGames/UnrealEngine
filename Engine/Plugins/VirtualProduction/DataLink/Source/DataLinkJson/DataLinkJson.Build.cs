// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataLinkJson : ModuleRules
{
    public DataLinkJson(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "DataLink",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Engine",
                "Json",
                "JsonUtilities",
            }
        );
    }
}