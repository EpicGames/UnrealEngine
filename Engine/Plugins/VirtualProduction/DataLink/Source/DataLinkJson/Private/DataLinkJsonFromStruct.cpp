// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkJsonFromStruct.h"
#include "DataLinkExecutor.h"
#include "DataLinkJsonNames.h"
#include "DataLinkNames.h"
#include "DataLinkNodeInstance.h"
#include "DataLinkPinBuilder.h"
#include "JsonObjectConverter.h"
#include "JsonObjectWrapper.h"

#define LOCTEXT_NAMESPACE "DataLinkJsonFromStruct"

void UDataLinkJsonFromStruct::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	Super::OnBuildPins(Inputs, Outputs);

	// No struct specified as this generically accepts any script struct and converts it to json object
	Inputs.Add(UE::DataLinkJson::InputStruct)
		.SetDisplayName(LOCTEXT("InputStructDisplay", "Struct"));

	Outputs.Add(UE::DataLink::OutputDefault)
		.SetDisplayName(LOCTEXT("InputJsonObjectDisplay", "Json"))
		.SetStruct<FJsonObjectWrapper>();
}

EDataLinkExecutionReply UDataLinkJsonFromStruct::OnExecute(FDataLinkExecutor& InExecutor) const
{
	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	FConstStructView InputStruct = NodeInstance.GetInputDataViewer().Find(UE::DataLinkJson::InputStruct);
	if (!InputStruct.IsValid())
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	FJsonObjectWrapper& Wrapper = NodeInstance.GetOutputDataViewer().Get<FJsonObjectWrapper>(UE::DataLink::OutputDefault);
	if (!Wrapper.JsonObject.IsValid())
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	const bool bResult = FJsonObjectConverter::UStructToJsonObject(InputStruct.GetScriptStruct()
		, InputStruct.GetMemory()
		, Wrapper.JsonObject.ToSharedRef());

	if (!bResult)
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	// Set the json object wrapper's string to the latest json object result
	Wrapper.JsonObjectToString(Wrapper.JsonString);

	InExecutor.Next(this);
	return EDataLinkExecutionReply::Handled;
}

#undef LOCTEXT_NAMESPACE
