// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkJsonLog.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogDataLinkJson);

IMPLEMENT_MODULE(FDefaultModuleImpl, DataLinkJson)
