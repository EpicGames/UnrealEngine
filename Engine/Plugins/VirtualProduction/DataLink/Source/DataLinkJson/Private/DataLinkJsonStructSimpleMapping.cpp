// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkJsonStructSimpleMapping.h"
#include "DataLinkJsonLog.h"
#include "DataLinkJsonUtils.h"
#include "DataLinkUtils.h"
#include "JsonObjectConverter.h"
#include "StructUtils/StructView.h"

bool UDataLinkJsonStructSimpleMapping::Apply(const TSharedRef<FJsonObject>& InSourceJson, const FStructView& InTargetStructView) const
{
	for (const TPair<FString, FName>& FieldMapping : FieldMappings)
	{
		FString ErrorMessage;
		const UE::DataLink::FPropertyView TargetPropertyView = UE::DataLink::ResolvePropertyView(InTargetStructView, FieldMapping.Value.ToString(), &ErrorMessage);
		if (!TargetPropertyView.Property)
		{
			UE_LOGF(LogDataLinkJson, Error, "Field '%ls' not found. %ls"
				, *FieldMapping.Value.ToString()
				, *ErrorMessage);
			continue;
		}

		const TSharedPtr<FJsonValue> JsonValue = UE::DataLinkJson::FindJsonValue(InSourceJson, FieldMapping.Key);
		if (!JsonValue.IsValid())
		{
			UE_LOGF(LogDataLinkJson, Error, "Field '%ls' not found in json"
				, *FieldMapping.Key);
			continue;
		}

		if (!FJsonObjectConverter::JsonValueToUProperty(JsonValue, TargetPropertyView.Property, TargetPropertyView.Memory))
		{
			UE_LOGF(LogDataLinkJson, Error, "Could not copy Json Value with key '%ls' to property '%ls' in struct '%ls'"
				, *FieldMapping.Key
				, *TargetPropertyView.Property->GetName()
				, *GetNameSafe(InTargetStructView.GetScriptStruct()));
		}
	}

	return true;
}
