// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkJsonToStruct.h"
#include "DataLinkExecutor.h"
#include "DataLinkJsonNames.h"
#include "DataLinkJsonStructMapping.h"
#include "DataLinkNames.h"
#include "DataLinkNodeInstance.h"
#include "DataLinkPinBuilder.h"
#include "JsonObjectConverter.h"
#include "JsonObjectWrapper.h"
#include "StructUtils/StructView.h"

#define LOCTEXT_NAMESPACE "DataLinkJsonToStruct"

void UDataLinkJsonToStruct::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	Super::OnBuildPins(Inputs, Outputs);

	Inputs.Add(UE::DataLinkJson::InputJsonObject)
		.SetDisplayName(LOCTEXT("InputJsonObjectDisplay", "Json"))
		.SetStruct<FJsonObjectWrapper>();

	Inputs.Add(UE::DataLinkJson::InputMappingConfig)
		.SetDisplayName(LOCTEXT("InputJsonStructMappingConfigDisplay", "Mapping Config"))
		.SetStruct<FDataLinkJsonStructMappingConfig>();

	// Output does not have a struct defined as it's defined via the Input Mapping Config
	Outputs.Add(UE::DataLink::OutputDefault)
		.SetDisplayName(LOCTEXT("OutputDisplay", "Struct"));
}

EDataLinkExecutionReply UDataLinkJsonToStruct::OnExecute(FDataLinkExecutor& InExecutor) const
{
	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	const FDataLinkInputDataViewer& InputDataViewer = NodeInstance.GetInputDataViewer();

	const FJsonObjectWrapper& InputData = InputDataViewer.Get<FJsonObjectWrapper>(UE::DataLinkJson::InputJsonObject);

	const FDataLinkJsonStructMappingConfig& MappingConfig = InputDataViewer.Get<FDataLinkJsonStructMappingConfig>(UE::DataLinkJson::InputMappingConfig);

	if (!InputData.JsonObject.IsValid() || !MappingConfig.OutputStruct)
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	bool bResult;

	const FStructView OutputDataView = NodeInstance.GetOutputDataViewer().Find(UE::DataLink::OutputDefault, MappingConfig.OutputStruct);

	if (MappingConfig.CustomMapping)
	{
		bResult = MappingConfig.CustomMapping->Apply(InputData.JsonObject.ToSharedRef(), OutputDataView);
	}
	else
	{
		bResult = FJsonObjectConverter::JsonObjectToUStruct(InputData.JsonObject.ToSharedRef()
			, OutputDataView.GetScriptStruct()
			, OutputDataView.GetMemory());
	}

	if (!bResult)
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	InExecutor.Next(this);
	return EDataLinkExecutionReply::Handled;
}

#undef LOCTEXT_NAMESPACE
