// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkJsonUtils.h"
#include "DataLinkJsonLog.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"

namespace UE::DataLinkJson::Private
{
	constexpr FStringView LeftArrayDelimiter = TEXT("[");
	constexpr FStringView RightArrayDelimiter = TEXT("]");

	void FindArraySubscripts(FStringView InStringView, TArray<FStringView>& OutSubscriptViews)
	{
		while (!InStringView.IsEmpty())
		{
			const int32 StartIndex = UE::String::FindFirst(InStringView, LeftArrayDelimiter, ESearchCase::CaseSensitive);
			if (StartIndex == INDEX_NONE)
			{
				return;
			}

			// Move string to the right so that it no longer has the left delimiter in view.
			InStringView.RightChopInline(StartIndex + LeftArrayDelimiter.Len());
			if (InStringView.IsEmpty())
			{
				return;
			}

			const int32 EndIndex = UE::String::FindFirst(InStringView, RightArrayDelimiter, ESearchCase::CaseSensitive);
			if (EndIndex == INDEX_NONE)
			{
				return;
			}

			OutSubscriptViews.Add(InStringView.Left(EndIndex));
			InStringView.RightChopInline(EndIndex + RightArrayDelimiter.Len());
		}
	}

	bool IterateArraySubscripts(TSharedPtr<FJsonValue>& InOutValue, TConstArrayView<FStringView> InArraySubscripts, const FString& InFieldName)
	{
		constexpr const TCHAR* ErrorPrefix = TEXT("Error while iterating Array Subscripts -");

		for (const FStringView& ArraySubscript : InArraySubscripts)
		{
			if (!InOutValue.IsValid())
			{
				UE_LOGF(LogDataLinkJson, Log, "%ls '%ls' Json Value is null.", ErrorPrefix, *InFieldName);
				return false;
			}

			TArray<TSharedPtr<FJsonValue>>* ValueArray;
			if (!InOutValue->TryGetArray(ValueArray))
			{
				UE_LOGF(LogDataLinkJson, Log, "%ls '%ls' is not an array!", ErrorPrefix, *InFieldName);
				return false;
			}

			check(ValueArray);
			FStringView::ElementType* End = nullptr;
			const int32 ArrayIndex = FCString::Strtoi(ArraySubscript.GetData(), &End, /*Base*/10);

			if (End != ArraySubscript.end())
			{
				UE_LOGF(LogDataLinkJson, Log, "%ls Array '%ls' with non-numeric subscript '%ls'.", ErrorPrefix, *InFieldName, *FString(ArraySubscript));
				return false;
			}

			if (!ValueArray->IsValidIndex(ArrayIndex))
			{
				UE_LOGF(LogDataLinkJson, Log, "%ls Array '%ls' does not have a valid index %d.", ErrorPrefix, *InFieldName, ArrayIndex);
				return false;
			}

			InOutValue = (*ValueArray)[ArrayIndex];
		}
		return true;
	}
}

TSharedPtr<FJsonValue> UE::DataLinkJson::FindJsonValue(const TSharedRef<FJsonObject>& InJsonObject, const FString& InFieldName)
{
	TArray<FString> FieldPath;
	InFieldName.ParseIntoArray(FieldPath, TEXT("."));

	// Rather than removing first item, reverse and remove from end
	Algo::Reverse(FieldPath);

	TSharedPtr<FJsonObject> CurrentObject = InJsonObject;
	while (!FieldPath.IsEmpty() && CurrentObject.IsValid())
	{
		const FString FieldName = FieldPath.Pop();

		FStringView FieldNameView = FieldName;

		TArray<FStringView> ArraySubscripts;
		Private::FindArraySubscripts(FieldName, ArraySubscripts);

		// If there were subscripts, only consider the left part of the string
		if (!ArraySubscripts.IsEmpty())
		{
			const int32 FirstSubscriptIndex = ArraySubscripts[0].GetData() - FieldNameView.GetData();
			FieldNameView.LeftInline(FirstSubscriptIndex - Private::LeftArrayDelimiter.Len());
		}

		TSharedPtr<FJsonValue> Value = CurrentObject->TryGetField(FieldNameView);
		if (!Value.IsValid())
		{
			UE_LOGF(LogDataLinkJson, Log, "Failed to find Json Value %ls.", *FieldName);
			return nullptr;
		}

		// Parse array if there are any subscripts (will skip if empty)
		if (!Private::IterateArraySubscripts(Value, ArraySubscripts, FieldName))
		{
			return nullptr;
		}

		// If last segment of the path, the value was found 
		if (FieldPath.IsEmpty())
		{
			return Value;
		}

		// Else, there's more segments to look into.
		// The value should be an object if there's going to be further diving
		CurrentObject = Value->AsObject();
	}

	return nullptr;
}
