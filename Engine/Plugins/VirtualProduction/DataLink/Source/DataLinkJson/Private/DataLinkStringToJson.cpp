// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkStringToJson.h"
#include "DataLinkCoreTypes.h"
#include "DataLinkExecutor.h"
#include "DataLinkJsonLog.h"
#include "DataLinkJsonNames.h"
#include "DataLinkNames.h"
#include "DataLinkNodeInstance.h"
#include "DataLinkPinBuilder.h"
#include "JsonObjectWrapper.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "StructUtils/StructView.h"

#define LOCTEXT_NAMESPACE "DataLinkStringToJson"

void UDataLinkStringToJson::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	Super::OnBuildPins(Inputs, Outputs);

	Inputs.Add(UE::DataLinkJson::InputString)
		.SetDisplayName(LOCTEXT("InputString", "String"))
		.SetStruct<FDataLinkString>();

	Outputs.Add(UE::DataLink::OutputDefault)
		.SetDisplayName(LOCTEXT("OutputDisplay", "Json"))
		.SetStruct<FJsonObjectWrapper>();
}

EDataLinkExecutionReply UDataLinkStringToJson::OnExecute(FDataLinkExecutor& InExecutor) const
{
	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	const FDataLinkInputDataViewer& InputDataViewer = NodeInstance.GetInputDataViewer();
	const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

	const FDataLinkString& InputData = InputDataViewer.Get<FDataLinkString>(UE::DataLinkJson::InputString);

	FJsonObjectWrapper& OutputData = OutputDataViewer.Get<FJsonObjectWrapper>(UE::DataLink::OutputDefault);

	// Return early if the output data's json string already matches the input string
	if (OutputData.JsonString == InputData.Value)
	{
		InExecutor.Next(this);
		return EDataLinkExecutionReply::Handled;
	}

	TSharedPtr<FJsonValue> JsonValue;

	// Deserialize JSON
	{
		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(InputData.Value);
		if (!FJsonSerializer::Deserialize(JsonReader, JsonValue))
		{
			UE_LOGF(LogDataLinkJson, Error, "[%.*ls] Failed to deserialize JSON from string %ls"
				, InExecutor.GetContextName().Len()
				, InExecutor.GetContextName().GetData()
				, *InputData.Value);
			return EDataLinkExecutionReply::Unhandled;
		}
	}

	const TArray<TSharedPtr<FJsonValue>>* JsonArray;
	if (JsonValue->TryGetArray(JsonArray) && ensure(JsonArray))
	{
		if (JsonArray->Num() != 1 || !(*JsonArray)[0].IsValid())
		{
			UE_LOGF(LogDataLinkJson, Error, "[%.*ls] Failed to find a valid JSON object from string %ls. Expected 1 valid json object in top level array. Found %d"
				, InExecutor.GetContextName().Len()
				, InExecutor.GetContextName().GetData()
				, *InputData.Value
				, JsonArray->Num());
			return EDataLinkExecutionReply::Unhandled;
		}

		OutputData.JsonObject = (*JsonArray)[0]->AsObject();
	}
	else
	{
		OutputData.JsonObject = JsonValue->AsObject();
	}

	if (!OutputData.JsonObject.IsValid())
	{
		UE_LOGF(LogDataLinkJson, Error, "[%.*ls] Failed to find a valid JSON object from string %ls"
			, InExecutor.GetContextName().Len()
			, InExecutor.GetContextName().GetData()
			, *InputData.Value);
		return EDataLinkExecutionReply::Unhandled;
	}

	// Save the Json String for future reference if the parsed object can be re-used for matching strings
	OutputData.JsonString = InputData.Value;
	InExecutor.Next(this);
	return EDataLinkExecutionReply::Handled;
}

#undef LOCTEXT_NAMESPACE
