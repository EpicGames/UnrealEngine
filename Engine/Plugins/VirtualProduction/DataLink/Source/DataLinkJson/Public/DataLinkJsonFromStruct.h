// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "DataLinkJsonFromStruct.generated.h"

/** Convert a struct to Json Object */
UCLASS(MinimalAPI, Category="JSON", DisplayName="Struct to JSON")
class UDataLinkJsonFromStruct : public UDataLinkNode
{
	GENERATED_BODY()

protected:
	//~ Begin UDataLinkNode
	DATALINKJSON_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
	DATALINKJSON_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
	//~ End UDataLinkNode
};
