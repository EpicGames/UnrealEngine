// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::DataLinkJson
{
	const FLazyName InputJsonObject(TEXT("InputJson"));
	const FLazyName InputMappingConfig(TEXT("InputMapping"));
	const FLazyName InputString(TEXT("InputString"));
	const FLazyName InputStruct(TEXT("InputStruct"));
}
