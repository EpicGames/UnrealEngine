// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include "UObject/Object.h"
#include "DataLinkJsonStructMapping.generated.h"

class FJsonObject;
struct FStructView;

/** Base class for any mapping logic from a json object to a given struct */
UCLASS(MinimalAPI, Abstract, EditInlineNew)
class UDataLinkJsonStructMapping : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Runs the mapping logic on the given Json object and the target struct view
	 * @param InSourceJson the json object to convert
	 * @param InTargetStructView the destination struct instance
	 * @return true if the mapping succeeded
	 */
	DATALINKJSON_API virtual bool Apply(const TSharedRef<FJsonObject>& InSourceJson, const FStructView& InTargetStructView) const
	{
		return false;
	}
};
