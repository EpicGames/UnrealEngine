// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkJsonStructMapping.h"
#include "DataLinkJsonStructSimpleMapping.generated.h"

UCLASS(MinimalAPI)
class UDataLinkJsonStructSimpleMapping : public UDataLinkJsonStructMapping
{
	GENERATED_BODY()

public:
	//~ Begin UDataLinkJsonStructMapping
	DATALINKJSON_API virtual bool Apply(const TSharedRef<FJsonObject>& InSourceJson, const FStructView& InTargetStructView) const override;
	//~ End UDataLinkJsonStructMapping

protected:
	/**
	 * Map for how a field in the json pairs to a given property name in a struct.
	 * Nested json fields can be accessed via a dot delimiter for each path segment.
	 */
	UPROPERTY(EditAnywhere, Category="Data Link")
	TMap<FString, FName> FieldMappings;
};
