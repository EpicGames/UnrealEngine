// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "DataLinkJsonToStruct.generated.h"

class UDataLinkJsonStructMapping;

USTRUCT(BlueprintType)
struct FDataLinkJsonStructMappingConfig
{
	GENERATED_BODY()

	/** The desired struct to convert the json object to */
	UPROPERTY(EditAnywhere, Category="Data Link")
	TObjectPtr<UScriptStruct> OutputStruct;

	/**
	 * Optional custom mapping to handle converting the Json object to the Output Struct
	 * If none is specified, the default mapping method will be used where the Struct property hierarchy should match that of the Json's
	 * @see FJsonObjectConverter::JsonObjectToUStruct
	 */
	UPROPERTY(EditAnywhere, Instanced, Category="Data Link")
	TObjectPtr<UDataLinkJsonStructMapping> CustomMapping;
};

/** Convert a Json Object to a particular struct */
UCLASS(MinimalAPI, Category="JSON", DisplayName="JSON to Struct")
class UDataLinkJsonToStruct : public UDataLinkNode
{
	GENERATED_BODY()

protected:
	//~ Begin UDataLinkNode
	DATALINKJSON_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
	DATALINKJSON_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
	//~ End UDataLinkNode
};
