// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class FJsonObject;
class FJsonValue;
class FString;

namespace UE::DataLinkJson
{
	/**
	 * Finds the json value that corresponds to the given field name in the given json object.
	 * @param InJsonObject the json object to look into
	 * @param InFieldName the field name to match. Can be in the format "A.B[2].C" to return nested values and array elements
	 * @return the json value if found
	 */
	DATALINKJSON_API TSharedPtr<FJsonValue> FindJsonValue(const TSharedRef<FJsonObject>& InJsonObject, const FString& InFieldName);
}
