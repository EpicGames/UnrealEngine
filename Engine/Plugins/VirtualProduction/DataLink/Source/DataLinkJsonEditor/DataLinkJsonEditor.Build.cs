// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataLinkJsonEditor : ModuleRules
{
    public DataLinkJsonEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
                "Json",
			}
		);

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "AssetTools",
                "BlueprintGraph",
                "ContentBrowser",
                "Core",
                "CoreUObject",
                "DataLinkEditor",
                "Engine",
                "JsonUtilities",
                "Slate",
                "SlateCore",
                "ToolMenus",
                "UnrealEd",
            }
        );
    }
}