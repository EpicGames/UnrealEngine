// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkJsonEditorMenu.h"
#include "DataLinkEditorNames.h"
#include "DataLinkJsonEditorStructGenerator.h"
#include "Dialogs/DlgPickPath.h"
#include "Framework/Application/SlateApplication.h"
#include "IDataLinkEditorMenuContext.h"
#include "JsonObjectWrapper.h"
#include "StructUtils/StructView.h"
#include "Styling/AppStyle.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Widgets/SDataLinkJsonEditorStructGeneratorConfig.h"

#define LOCTEXT_NAMESPACE "DataLinkJsonEditorMenu"

void UE::DataLinkJsonEditor::RegisterMenus()
{
	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(UE::DataLinkEditor::PreviewToolbarName);
	if (!ToolMenu)
	{
		return;
	}

	ToolMenu->AddDynamicSection(TEXT("JsonEditor")
		, FNewToolMenuDelegate::CreateStatic(&UE::DataLinkJsonEditor::PopulateToolbar)
		, FToolMenuInsert(UE::DataLinkEditor::PreviewSectionName, EToolMenuInsertType::After));
}

void UE::DataLinkJsonEditor::PopulateToolbar(UToolMenu* InToolMenu)
{
	FToolUIAction MakeStructsAction;
	MakeStructsAction.ExecuteAction.BindStatic(&UE::DataLinkJsonEditor::MakeStructsFromJson);
	MakeStructsAction.CanExecuteAction.BindStatic(&UE::DataLinkJsonEditor::CanMakeStructsFromJson);

	FToolMenuSection& Section = InToolMenu->AddSection(TEXT("Json"));
	Section.AddEntry(FToolMenuEntry::InitToolBarButton(TEXT("MakeStructs")
		, MakeStructsAction
		, LOCTEXT("MakeStructsDisplayName", "Make Structs from Json")
		, LOCTEXT("Tooltip", "Makes struct assets matching the Json hierarchy of the Output Data (needs to be Json)")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Toolbar.Export"))));
}

TConstStructView<FJsonObjectWrapper> UE::DataLinkJsonEditor::GetPreviewOutputDataView(const IDataLinkEditorMenuContext& InMenuContext)
{
	FConstStructView PreviewOutputData = InMenuContext.FindPreviewOutputData();
	if (!PreviewOutputData.IsValid() || PreviewOutputData.GetScriptStruct() != FJsonObjectWrapper::StaticStruct())
	{
		return TConstStructView<FJsonObjectWrapper>();
	}

	return TConstStructView<FJsonObjectWrapper>(PreviewOutputData.GetMemory());
}

bool UE::DataLinkJsonEditor::CanMakeStructsFromJson(const FToolMenuContext& InMenuContext)
{
	IDataLinkEditorMenuContext* const MenuContext = InMenuContext.FindContext<IDataLinkEditorMenuContext>();
	if (!MenuContext)
	{
		return false;
	}

	TConstStructView<FJsonObjectWrapper> JsonWrapper = GetPreviewOutputDataView(*MenuContext);
	return JsonWrapper.IsValid() && JsonWrapper.Get().JsonObject.IsValid();
}

void UE::DataLinkJsonEditor::MakeStructsFromJson(const FToolMenuContext& InMenuContext)
{
	IDataLinkEditorMenuContext* const MenuContext = InMenuContext.FindContext<IDataLinkEditorMenuContext>();
	if (!MenuContext)
	{
		return;
	}

	TConstStructView<FJsonObjectWrapper> JsonWrapper = GetPreviewOutputDataView(*MenuContext);
	if (!JsonWrapper.IsValid())
	{
		return;
	}

	TSharedPtr<FJsonObject> JsonObject = JsonWrapper.Get().JsonObject;
	if (!JsonObject.IsValid())
	{
		return;
	}

	FSlateApplication::Get().AddWindow(SNew(SDataLinkJsonEditorStructGeneratorConfig)
		.Title(LOCTEXT("ChooseTargetContentPath", "Choose Location for the Structs"))
		.DefaultPath(MenuContext->GetAssetPath())
		.OnCommit_Lambda(
			[JsonObject = MoveTemp(JsonObject)](const SDataLinkJsonEditorStructGeneratorConfig& InConfig)
			{
				FStructGenerator::FParams Params;
				Params.JsonObject = JsonObject;
				Params.BasePath = InConfig.GetPath();
				Params.StructPrefix = InConfig.GetPrefix();
				Params.RootStructName = InConfig.GetRootStructName();
				FStructGenerator::GenerateFromJson(Params);
			}));
}

#undef LOCTEXT_NAMESPACE
