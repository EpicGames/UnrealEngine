// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FJsonObject;
class IDataLinkEditorMenuContext;
class UToolMenu;
struct FJsonObjectWrapper;
struct FToolMenuContext;
template<typename BaseStructT> struct TConstStructView;

namespace UE::DataLinkJsonEditor
{
	void RegisterMenus();

	void PopulateToolbar(UToolMenu* InToolMenu);

	TConstStructView<FJsonObjectWrapper> GetPreviewOutputDataView(const IDataLinkEditorMenuContext& InMenuContext);

	bool CanMakeStructsFromJson(const FToolMenuContext& InMenuContext);

	void MakeStructsFromJson(const FToolMenuContext& InMenuContext);
}
