// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkJsonEditorModule.h"
#include "DataLinkEditorNames.h"
#include "DataLinkJsonEditorLog.h"
#include "DataLinkJsonEditorMenu.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"

DEFINE_LOG_CATEGORY(LogDataLinkJsonEditor);

IMPLEMENT_MODULE(FDataLinkJsonEditorModule, DataLinkJsonEditor)

void FDataLinkJsonEditorModule::StartupModule()
{
	FSimpleMulticastDelegate::FDelegate OnToolMenusStartup;
	OnToolMenusStartup.BindStatic(&UE::DataLinkJsonEditor::RegisterMenus);

	OnToolMenusStartupHandle = UToolMenus::RegisterStartupCallback(OnToolMenusStartup);
}

void FDataLinkJsonEditorModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(OnToolMenusStartupHandle);
	OnToolMenusStartupHandle.Reset();
}
