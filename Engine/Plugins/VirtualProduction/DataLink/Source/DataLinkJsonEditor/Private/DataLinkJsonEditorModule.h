// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"

class FDataLinkJsonEditorModule : public IModuleInterface
{
public:
    //~ Begin IModuleInterface
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
    //~ End IModuleInterface

private:
    FDelegateHandle OnToolMenusStartupHandle;
};
