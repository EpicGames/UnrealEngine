// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkJsonEditorStructGenerator.h"
#include "AssetToolsModule.h"
#include "DataLinkJsonEditorLog.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "EdGraphSchema_K2.h"
#include "Factories/StructureFactory.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "Kismet2/StructureEditorUtils.h"
#include "PackageTools.h"
#include "ScopedTransaction.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"

#define LOCTEXT_NAMESPACE "DataLinkJsonEditorUtils"

UE::DataLinkJsonEditor::FStructKey::FStructKey(FStructGenerator& InStructGenerator, const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& InJsonEntries)
{
	Hash = 0;
	JsonTypeMap.Reserve(InJsonEntries.Num());
	for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonEntry : InJsonEntries)
	{
		FEdGraphPinType Type;
		FString Key(JsonEntry.Key);
		if (InStructGenerator.FromJsonValue(JsonEntry.Value, Type, Key))
		{
			Hash = HashCombine(Hash, GetTypeHash(Key));
			Hash = HashCombine(Hash, GetTypeHash(Type.PinCategory));
			Hash = HashCombine(Hash, GetTypeHash(Type.PinSubCategoryObject));
			Hash = HashCombine(Hash, GetTypeHash(Type.ContainerType));

			JsonTypeMap.Add(Key, Type);
		}
	}
}

void UE::DataLinkJsonEditor::FStructGenerator::GenerateFromJson(const FParams& InParams)
{
	FScopedSlowTask SlowTask(0, LOCTEXT("SlowTaskText", "Generating structs from Json..."));
	SlowTask.MakeDialog();

	// Note: this transaction is only here because FStructureEditorUtils CreateAsset for User Defined Structs transact
	// because of FStructureEditorUtils::AddVariable. So this is here to avoid multiple transactions and control it from here.
	// This transaction will be cancelled at the end as asset creation should've not transacted in the first place.
	FScopedTransaction Transaction(LOCTEXT("Transaction", "Generate Structs from Json"));

	FStructGenerator Generator;
	Generator.BasePath = InParams.BasePath;
	Generator.StructPrefix = InParams.StructPrefix;
	Generator.GetOrCreateStruct(InParams.JsonObject, InParams.RootStructName);

	Transaction.Cancel();

	IContentBrowserSingleton::Get().SyncBrowserToAssets(Generator.GetGeneratedStructs());
}

bool UE::DataLinkJsonEditor::FStructGenerator::FromJsonValue(const TSharedPtr<FJsonValue>& InJsonValue, FEdGraphPinType& OutPinType, const FString& InNameToUse)
{
	if (!InJsonValue.IsValid())
	{
		UE_LOGF(LogDataLinkJsonEditor, Error, "Invalid Json Value '%ls'", *InNameToUse);
		return false;
	}

	switch (InJsonValue->Type)
	{
	case EJson::String:
		OutPinType.PinCategory = UEdGraphSchema_K2::PC_String;
		return true;

	case EJson::Number:
		// default to real as seen in `FJsonValue::AsNumber`
		OutPinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		OutPinType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
		return true;

	case EJson::Boolean:
		OutPinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
		return true;

	case EJson::Array:
		{
			const TArray<TSharedPtr<FJsonValue>>& JsonArray = InJsonValue->AsArray();
			if (JsonArray.IsEmpty() || !JsonArray[0].IsValid())
			{
				UE_LOGF(LogDataLinkJsonEditor, Error, "Empty array '%ls' is not supported!", *InNameToUse);
				return false;
			}

			// Nested arrays are not supported.
			// A middle struct can be created, but the json utils to import/export structs would not produce same hierarchy as the imported json
			if (JsonArray[0]->Type == EJson::Array)
			{
				UE_LOGF(LogDataLinkJsonEditor, Error, "Nested array '%ls' is not supported!", *InNameToUse);
				return false;
			}

			// Use the first element as sample to get the inner array type
			if (FromJsonValue(JsonArray[0], OutPinType, InNameToUse))
			{
				OutPinType.ContainerType = EPinContainerType::Array;
				return true;
			}
		}
		UE_LOGF(LogDataLinkJsonEditor, Error, "Could not create a valid pin array type for '%ls'", *InNameToUse);
		return false;

	case EJson::Object:
		if (UUserDefinedStruct* Struct = GetOrCreateStruct(InJsonValue->AsObject(), InNameToUse))
		{
			OutPinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
			OutPinType.PinSubCategoryObject = Struct;
			return true;
		}
		UE_LOGF(LogDataLinkJsonEditor, Error, "Could not create a valid struct type for '%ls'", *InNameToUse);
		return false;
	}

	UE_LOGF(LogDataLinkJsonEditor, Error, "Unsupported Json type encountered at '%ls'", *InNameToUse);
	return false;
}

void UE::DataLinkJsonEditor::FStructGenerator::AddVariable(UUserDefinedStruct* InStruct, const FString& InName, const TSharedPtr<FJsonValue>& InValue)
{
	FEdGraphPinType PinType;
	if (!FromJsonValue(InValue, PinType, InName))
	{
		return;
	}

	const FGuid Guid = FGuid::NewGuid();

	FStructVariableDescription VarDesc;
	VarDesc.VarName = *FString::Printf(TEXT("%s_%s"), *InName, *Guid.ToString(EGuidFormats::Digits));
	VarDesc.VarGuid = Guid;
	VarDesc.FriendlyName = InName;
	VarDesc.SetPinType(PinType);

	FStructureEditorUtils::GetVarDesc(InStruct).Add(MoveTemp(VarDesc));
}

UUserDefinedStruct* UE::DataLinkJsonEditor::FStructGenerator::CreateEmptyStruct(const FString& InNameToUse)
{
	if (InNameToUse.IsEmpty())
	{
		UE_LOGF(LogDataLinkJsonEditor, Error, "Could not create a valid struct. Empty name!");
		return nullptr;
	}

	if (!StructureFactory)
	{
		StructureFactory = NewObject<UStructureFactory>();
	}

	if (!FName::IsValidXName(InNameToUse, INVALID_OBJECTNAME_CHARACTERS))
	{
		const FString CleanName = MakeObjectNameFromDisplayLabel(InNameToUse, NAME_None).GetPlainNameString();
		UE_LOGF(LogDataLinkJsonEditor, Log, "Input Struct name '%ls' is not valid. Cleaned name to '%ls'", *InNameToUse, *CleanName);
		return CreateEmptyStruct(CleanName);
	}

	UUserDefinedStruct* const Struct = Cast<UUserDefinedStruct>(IAssetTools::Get().CreateAsset(InNameToUse
		, BasePath
		, UUserDefinedStruct::StaticClass()
		, StructureFactory));

	if (Struct)
	{
		FStructureEditorUtils::GetVarDesc(Struct).Empty();
	}

	return Struct;
}

UUserDefinedStruct* UE::DataLinkJsonEditor::FStructGenerator::GetOrCreateStruct(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InNameToUse)
{
	if (!InJsonObject.IsValid() || InJsonObject->Values.IsEmpty())
	{
		UE_LOGF(LogDataLinkJsonEditor, Error, "Could not create a valid struct. Json Object: '%ls'", *InNameToUse);
		return nullptr;
	}

	const FStructKey StructKey(*this, InJsonObject->Values);
	if (const TObjectPtr<UUserDefinedStruct>* FoundStruct = GeneratedStructs.Find(StructKey))
	{
		// This could return null for structs that failed to create
		return *FoundStruct;
	}

	UUserDefinedStruct* Struct = CreateEmptyStruct(StructPrefix + InNameToUse);

	// Add even if empty. There was an attempt to create the struct, but it failed. Avoid repeating the same steps for
	// structs with the same schema
	GeneratedStructs.Add(StructKey, Struct);

	if (!Struct)
	{
		return nullptr;
	}

	for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonEntry : InJsonObject->Values)
	{
		AddVariable(Struct, FString(JsonEntry.Key), JsonEntry.Value);
	}

	FStructureEditorUtils::CompileStructure(Struct);
	return Struct;
}

TArray<UObject*> UE::DataLinkJsonEditor::FStructGenerator::GetGeneratedStructs() const
{
	TArray<UObject*> Structs;
	Structs.Reserve(GeneratedStructs.Num());

	for (const TPair<FStructKey, TObjectPtr<UUserDefinedStruct>>& Pair : GeneratedStructs)
	{
		Structs.Add(Pair.Value);
	}

	return Structs;
}

#undef LOCTEXT_NAMESPACE
