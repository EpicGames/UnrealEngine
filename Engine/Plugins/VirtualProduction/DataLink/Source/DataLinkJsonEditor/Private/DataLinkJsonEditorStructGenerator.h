// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dom/JsonObject.h"
#include "EdGraph/EdGraphPin.h"
#include "Templates/SharedPointerFwd.h"

class FJsonObject;
class FJsonValue;
class UUserDefinedStruct;
class UStructureFactory;

namespace UE::DataLinkJsonEditor
{

class FStructGenerator;

/** Identifies a map of Name to Json Value to identify a struct that can be re-used */
struct FStructKey
{
	explicit FStructKey(FStructGenerator& InStructGenerator, const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& InJsonEntries);

	bool operator==(const FStructKey& InOther) const
	{
		return Hash == InOther.Hash && JsonTypeMap.OrderIndependentCompareEqual(InOther.JsonTypeMap);
	}

	friend uint32 GetTypeHash(const FStructKey& InKey)
	{
		return InKey.Hash;
	}

private:
	uint32 Hash = 0;
	TMap<FString, FEdGraphPinType> JsonTypeMap;
};

/** Generates as many structs as needed to create a hierarchy that matches the Json Object*/
class FStructGenerator
{
	friend struct FStructKey;

public:
	struct FParams
	{
		TSharedPtr<FJsonObject> JsonObject;
		FString BasePath;
		FString StructPrefix;
		FString RootStructName;
	};
	static void GenerateFromJson(const FParams& InParams);

private:
	bool FromJsonValue(const TSharedPtr<FJsonValue>& InJsonValue, FEdGraphPinType& OutPinType, const FString& InNameToUse);

	void AddVariable(UUserDefinedStruct* InStruct, const FString& InName, const TSharedPtr<FJsonValue>& InValue);

	UUserDefinedStruct* CreateEmptyStruct(const FString& InNameToUse);

	UUserDefinedStruct* GetOrCreateStruct(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InNameToUse);

	TArray<UObject*> GetGeneratedStructs() const;

	TMap<FStructKey, TObjectPtr<UUserDefinedStruct>> GeneratedStructs;

	TObjectPtr<UStructureFactory> StructureFactory = nullptr;

	FString BasePath;

	FString StructPrefix;
};

}
