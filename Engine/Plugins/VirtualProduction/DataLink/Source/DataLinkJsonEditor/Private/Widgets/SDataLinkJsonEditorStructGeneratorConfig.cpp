// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataLinkJsonEditorStructGeneratorConfig.h"
#include "ContentBrowserModule.h"
#include "DetailLayoutBuilder.h"
#include "IContentBrowserSingleton.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SDataLinkJsonEditorStructGeneratorConfig"

void SDataLinkJsonEditorStructGeneratorConfig::Construct(const FArguments& InArgs)
{
	Path = InArgs._DefaultPath;
	RootStructName = TEXT("RootStruct");
	OnCommitDelegate = InArgs._OnCommit;

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = Path;
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SDataLinkJsonEditorStructGeneratorConfig::OnPathChange);
	PathPickerConfig.bAddDefaultPath = true;
	PathPickerConfig.bAllowReadOnlyFolders = false;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	SWindow::Construct(SWindow::FArguments()
		.Title(InArgs._Title)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.ClientSize(FVector2D(450,450))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(2,2,2,4)
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(1.f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("PrefixLabel", "Prefix"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(SEditableTextBox)
						.HintText(LOCTEXT("PrefixHint", "Enter the string to use as prefix for each struct name."))
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.OnTextChanged(this, &SDataLinkJsonEditorStructGeneratorConfig::OnPrefixChange)
					]
				]
			]
			+ SVerticalBox::Slot()
			.Padding(2,2,2,4)
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(1.f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("RootStructLabel", "Root Struct Name"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(SEditableTextBox)
						.Text(FText::FromString(RootStructName))
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.OnTextChanged(this, &SDataLinkJsonEditorStructGeneratorConfig::OnRootStructNameChange)
					]
				]
			]
			+ SVerticalBox::Slot()
			.Padding(2,2,2,4)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(3.f)
				[
					ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SButton)
					.Text(LOCTEXT("OK", "OK"))
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SDataLinkJsonEditorStructGeneratorConfig::OnCommit)
				]
				+SUniformGridPanel::Slot(1,0)
				[
					SNew(SButton)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SDataLinkJsonEditorStructGeneratorConfig::OnCancel)
				]
			]
		]);
}

const FString& SDataLinkJsonEditorStructGeneratorConfig::GetPath() const
{
	return Path;
}

const FString& SDataLinkJsonEditorStructGeneratorConfig::GetPrefix() const
{
	return Prefix;
}

const FString& SDataLinkJsonEditorStructGeneratorConfig::GetRootStructName() const
{
	return RootStructName;
}

FReply SDataLinkJsonEditorStructGeneratorConfig::OnCommit()
{
	if (ValidateConfiguration())
	{
		OnCommitDelegate.ExecuteIfBound(*this);
		RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SDataLinkJsonEditorStructGeneratorConfig::OnCancel()
{
	RequestDestroyWindow();
	return FReply::Handled();
}

void SDataLinkJsonEditorStructGeneratorConfig::OnPathChange(const FString& InNewPath)
{
	Path = InNewPath;
}

void SDataLinkJsonEditorStructGeneratorConfig::OnPrefixChange(const FText& InNewPrefix)
{
	Prefix = InNewPrefix.ToString();
}

void SDataLinkJsonEditorStructGeneratorConfig::OnRootStructNameChange(const FText& InNewRootName)
{
	RootStructName = InNewRootName.ToString();
}

bool SDataLinkJsonEditorStructGeneratorConfig::ValidateConfiguration()
{
	if (Path.IsEmpty())
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("EmptyPath", "You must select a path."));
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
