// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"

class SDataLinkJsonEditorStructGeneratorConfig : public SWindow
{
public:
	DECLARE_DELEGATE_OneParam(FOnCommit, const SDataLinkJsonEditorStructGeneratorConfig&);

	SLATE_BEGIN_ARGS(SDataLinkJsonEditorStructGeneratorConfig) {}
	SLATE_ARGUMENT(FText, Title)
	SLATE_ARGUMENT(FString, DefaultPath)
	SLATE_EVENT(FOnCommit, OnCommit)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	const FString& GetPath() const;
	const FString& GetPrefix() const;
	const FString& GetRootStructName() const;

private:
	FReply OnCommit();
	FReply OnCancel();

	void OnPathChange(const FString& InNewPath);

	void OnPrefixChange(const FText& InNewPrefix);

	void OnRootStructNameChange(const FText& InNewRootName);

	bool ValidateConfiguration();

	FString Path;

	FString Prefix;

	FString RootStructName;

	FOnCommit OnCommitDelegate;
};
