// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataLinkWebSocket : ModuleRules
{
    public DataLinkWebSocket(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "DataLink",
                "Engine",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "WebSockets",
            }
        );
    }
}
