// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkWebSocket.h"
#include "DataLinkCoreTypes.h"
#include "DataLinkExecutor.h"
#include "DataLinkNames.h"
#include "DataLinkNodeInstance.h"
#include "DataLinkPinBuilder.h"
#include "DataLinkWebSocketLog.h"
#include "DataLinkWebSocketNames.h"
#include "DataLinkWebSocketSettings.h"
#include "DataLinkWebSocketSubsystem.h"
#include "IWebSocket.h"

#define LOCTEXT_NAMESPACE "DataLinkWebSocket"

UDataLinkWebSocket::UDataLinkWebSocket()
{
	InstanceStruct = FDataLinkWebSocketInstanceData::StaticStruct();
}

void UDataLinkWebSocket::OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const
{
	Super::OnBuildPins(Inputs, Outputs);

	Inputs.Add(UE::DataLink::InputWebSocketSettings)
		.SetDisplayName(LOCTEXT("WebSocketSettingsDisplay", "Web Socket Settings"))
		.SetStruct<FDataLinkWebSocketSettings>();

	Inputs.Add(UE::DataLink::InputWebSocketMessages)
		.SetDisplayName(LOCTEXT("WebSocketMessagesDisplay", "Connect Messages"))
		.SetStruct<FDataLinkWebSocketMessages>();

	Outputs.Add(UE::DataLink::OutputDefault)
		.SetDisplayName(LOCTEXT("OutputDisplay", "Message Received"))
		.SetStruct<FDataLinkString>();
}

EDataLinkExecutionReply UDataLinkWebSocket::OnExecute(FDataLinkExecutor& InExecutor) const
{
	UDataLinkWebSocketSubsystem* WebSocketSubsystem = UDataLinkWebSocketSubsystem::TryGet();
	if (!WebSocketSubsystem)
	{
		return EDataLinkExecutionReply::Unhandled;
	}

	FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstanceMutable(this);
	FDataLinkWebSocketInstanceData& InstanceData = NodeInstance.GetInstanceDataMutable().Get<FDataLinkWebSocketInstanceData>();

	const FDataLinkInputDataViewer& InputDataViewer = NodeInstance.GetInputDataViewer();

	const FDataLinkWebSocketSettings& WebSocketSettings = InputDataViewer.Get<FDataLinkWebSocketSettings>(UE::DataLink::InputWebSocketSettings);

	TSharedPtr<IWebSocket> WebSocket = WebSocketSubsystem->FindWebSocket(InstanceData.WebSocketHandle);

	if (WebSocket.IsValid() && WebSocket->IsConnected())
	{
		// If current web socket is valid and connected, early exit only if the web socket settings are the same.
		if (InstanceData.WebSocketSettings.Equals(WebSocketSettings))
		{
			return EDataLinkExecutionReply::Handled;
		}

		UE_LOGF(LogDataLinkWebSocket, Verbose, "[%.*ls] Existing Web Socket found but connection settings differ from requested. Closing web socket."
			, InExecutor.GetContextName().Len(), InExecutor.GetContextName().GetData());
	}

	// Close web socket (existing is either non-existent, invalid, or connected but its connection settings are different)
	if (InstanceData.WebSocketHandle.IsValid())
	{
		WebSocketSubsystem->CloseWebSocket(InstanceData.WebSocketHandle);
		InstanceData.WebSocketHandle.Reset();
	}

	InstanceData.WebSocketSettings.Reset();

	// Create Web Socket
	{
		UDataLinkWebSocketSubsystem::FCreateWebSocketResult CreateWebSocketResult;
		if (!WebSocketSubsystem->CreateWebSocket(WebSocketSettings, CreateWebSocketResult))
		{
			UE_LOGF(LogDataLinkWebSocket, Error, "[%.*ls] Failed to create web socket", InExecutor.GetContextName().Len(), InExecutor.GetContextName().GetData());
			return EDataLinkExecutionReply::Unhandled;
		}

		WebSocket = MoveTemp(CreateWebSocketResult.WebSocket);
		InstanceData.WebSocketHandle = MoveTemp(CreateWebSocketResult.Handle);
	}

	const TWeakPtr<FDataLinkExecutor> ExecutorWeak = InExecutor.AsWeak();

	WebSocket->OnConnected().AddUObject(this, &UDataLinkWebSocket::OnConnected, ExecutorWeak);
	WebSocket->OnConnectionError().AddUObject(this, &UDataLinkWebSocket::OnConnectionError, ExecutorWeak);
	WebSocket->OnClosed().AddUObject(this, &UDataLinkWebSocket::OnClosed, ExecutorWeak);
	WebSocket->OnMessage().AddUObject(this, &UDataLinkWebSocket::OnMessageReceived, ExecutorWeak);
	WebSocket->Connect();

	return EDataLinkExecutionReply::Handled;
}

void UDataLinkWebSocket::OnStop(const FDataLinkExecutor& InExecutor) const
{
	UDataLinkWebSocketSubsystem* WebSocketSubsystem = UDataLinkWebSocketSubsystem::TryGet();
	if (!WebSocketSubsystem)
	{
		return;
	}

	const FDataLinkNodeInstance& NodeInstance = InExecutor.GetNodeInstance(this);

	const FDataLinkWebSocketInstanceData& InstanceData = NodeInstance.GetInstanceData().Get<const FDataLinkWebSocketInstanceData>();

	if (TSharedPtr<IWebSocket> WebSocket = WebSocketSubsystem->FindWebSocket(InstanceData.WebSocketHandle))
	{
		WebSocket->OnClosed().RemoveAll(this);
	}

	WebSocketSubsystem->CloseWebSocket(InstanceData.WebSocketHandle);
}

void UDataLinkWebSocket::OnConnected(TWeakPtr<FDataLinkExecutor> InExecutorWeak) const
{
	UDataLinkWebSocketSubsystem* WebSocketSubsystem = UDataLinkWebSocketSubsystem::TryGet();
	if (!WebSocketSubsystem)
	{
		return;
	}

	TSharedPtr<FDataLinkExecutor> Executor = InExecutorWeak.Pin();
	if (!Executor.IsValid())
	{
		return;
	}

	const FDataLinkNodeInstance& NodeInstance = Executor->GetNodeInstance(this);
	const FDataLinkWebSocketInstanceData& InstanceData = NodeInstance.GetInstanceData().Get<const FDataLinkWebSocketInstanceData>();

	TSharedPtr<IWebSocket> WebSocket = WebSocketSubsystem->FindWebSocket(InstanceData.WebSocketHandle);
	if (!WebSocket.IsValid())
	{
		Executor->Fail(this);
		return;
	}

	const FDataLinkInputDataViewer& InputDataViewer = NodeInstance.GetInputDataViewer();
	const FDataLinkWebSocketMessages& WebSocketMessages = InputDataViewer.Get<FDataLinkWebSocketMessages>(UE::DataLink::InputWebSocketMessages);

	for (const FString& Message : WebSocketMessages.ConnectMessages)
	{
		WebSocket->Send(Message);
	}
}

void UDataLinkWebSocket::OnConnectionError(const FString& InError, TWeakPtr<FDataLinkExecutor> InExecutorWeak) const
{
	TSharedPtr<FDataLinkExecutor> Executor = InExecutorWeak.Pin();
	if (!Executor.IsValid())
	{
		return;
	}

	UE_LOGF(LogDataLinkWebSocket, Error, "[%.*ls] Connection Error: %ls"
		, Executor->GetContextName().Len(), Executor->GetContextName().GetData()
		, *InError);

	Executor->Fail(this);
}

void UDataLinkWebSocket::OnClosed(int32 InStatusCode, const FString& InReason, bool bInWasClean, TWeakPtr<FDataLinkExecutor> InExecutorWeak) const
{
	TSharedPtr<FDataLinkExecutor> Executor = InExecutorWeak.Pin();
	if (!Executor.IsValid())
	{
		return;
	}

	UE_LOGF(LogDataLinkWebSocket, Error, "[%.*ls] Web Socket Closed Status: %d --- %ls"
		, Executor->GetContextName().Len(), Executor->GetContextName().GetData()
		, InStatusCode
		, *InReason);

	Executor->Fail(this);
}

void UDataLinkWebSocket::OnMessageReceived(const FString& InMessage, TWeakPtr<FDataLinkExecutor> InExecutorWeak) const
{
	TSharedPtr<FDataLinkExecutor> Executor = InExecutorWeak.Pin();
	if (!Executor.IsValid())
	{
		return;
	}

	const FDataLinkNodeInstance& NodeInstance = Executor->GetNodeInstance(this);
	const FDataLinkOutputDataViewer& OutputDataViewer = NodeInstance.GetOutputDataViewer();

	FDataLinkString& OutputDataView = OutputDataViewer.Get<FDataLinkString>(UE::DataLink::OutputDefault);
	OutputDataView.Value = InMessage;

	// Keep execution running when message is received
	Executor->NextPersist(this);
}

#undef LOCTEXT_NAMESPACE
