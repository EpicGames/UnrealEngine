// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkWebSocketHandle.h"

namespace UE::DataLink
{

FWebSocketHandle FWebSocketHandle::GenerateNewHandle()
{
	static uint32 NextId = 0;

	FWebSocketHandle Handle;

	// Start from 1
	Handle.Id = ++NextId;

	// Handle edge case where NextId wraps around to 0.
	// 0 is reserved for invalid handles.
	if (Handle.Id == 0)
	{
		Handle.Id = ++NextId;
	}

	return Handle;
}

} // UE::DataLink
