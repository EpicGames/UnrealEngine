// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkWebSocketSettings.h"

void FDataLinkWebSocketSettings::Reset()
{
	URL.Reset();
	Protocols.Reset();
	UpgradeHeaders.Reset();
}

bool FDataLinkWebSocketSettings::Equals(const FDataLinkWebSocketSettings& InOther) const
{
	constexpr ESearchCase::Type SearchCase = ESearchCase::CaseSensitive;

	if (!URL.Equals(InOther.URL, SearchCase))
	{
		return false;
	}

	if (Protocols.Num() != InOther.Protocols.Num() || UpgradeHeaders.Num() != InOther.UpgradeHeaders.Num())
	{
		return false;
	}

	for (int32 Index = 0; Index < Protocols.Num(); ++Index)
	{
		if (!Protocols[Index].Equals(InOther.Protocols[Index], SearchCase))
		{
			return false;
		}
	}

	for (const TPair<FString, FString>& HeaderPair : UpgradeHeaders)
	{
		const FString* HeaderValue = InOther.UpgradeHeaders.Find(HeaderPair.Key);
		if (!HeaderValue || !HeaderPair.Value.Equals(*HeaderValue, SearchCase))
		{
			return false;
		}
	}

	return true;
}
