// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataLinkWebSocketSubsystem.h"
#include "DataLinkWebSocketLog.h"
#include "DataLinkWebSocketSettings.h"
#include "Engine/Engine.h"
#include "IWebSocket.h"
#include "WebSocketsModule.h"

UDataLinkWebSocketSubsystem* UDataLinkWebSocketSubsystem::TryGet()
{
	if (GEngine)
	{
		return GEngine->GetEngineSubsystem<UDataLinkWebSocketSubsystem>();
	}
	return nullptr;
}

bool UDataLinkWebSocketSubsystem::CreateWebSocket(const FDataLinkWebSocketSettings& InSettings, FCreateWebSocketResult& OutResult)
{
#if WITH_WEBSOCKETS
	const UE::DataLink::FWebSocketHandle WebSocketHandle = UE::DataLink::FWebSocketHandle::GenerateNewHandle();

	TSharedRef<IWebSocket> WebSocket = FWebSocketsModule::Get().CreateWebSocket(InSettings.URL, InSettings.Protocols, InSettings.UpgradeHeaders);
	WebSockets.Add(WebSocketHandle, WebSocket);

	OutResult.WebSocket = WebSocket;
	OutResult.Handle = WebSocketHandle;
	return true;
#else
	UE_LOGF(LogDataLinkWebSocket, Error, "Failed to create web socket. Web Sockets are disabled.");
	return false;
#endif
}

void UDataLinkWebSocketSubsystem::CloseWebSocket(UE::DataLink::FWebSocketHandle InHandle)
{
	TSharedPtr<IWebSocket> WebSocket = FindWebSocket(InHandle);
	if (!WebSocket.IsValid())
	{
		return;
	}

	WebSocket->OnClosed().AddWeakLambda(this,
		[this, InHandle](int32 /*StatusCode*/, const FString& /*Reason*/, bool /*bWasClean*/)
		{
			check(IsInGameThread());

			InvalidWebSockets.Add(WebSockets[InHandle]);
			WebSockets.Remove(InHandle);

			if (!CleanupHandle.IsValid())
			{
				constexpr float CleanupDelay = 1.f;

				CleanupHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this,
					[This=this](float)
					{
						This->CleanupInvalidWebSockets();
						return false; // one time call
					})
					, CleanupDelay);
			}
		});

	WebSocket->Close();
}

TSharedPtr<IWebSocket> UDataLinkWebSocketSubsystem::FindWebSocket(UE::DataLink::FWebSocketHandle InHandle) const
{
	if (!InHandle.IsValid())
	{
		return nullptr;
	}

	if (const TSharedRef<IWebSocket>* WebSocket = WebSockets.Find(InHandle))
	{
		return *WebSocket;
	}

	return nullptr;
}

void UDataLinkWebSocketSubsystem::CleanupInvalidWebSockets()
{
	check(IsInGameThread());

	// Removing CleanupHandle Ticker while still executing this ticker delegate is by-design valid.
	// see TickerTests.cpp:112 "a delegate removal from inside the delegate execution (used to be a deadlock)"
	FTSTicker::RemoveTicker(CleanupHandle);
	CleanupHandle.Reset();

	InvalidWebSockets.Reset();
}
