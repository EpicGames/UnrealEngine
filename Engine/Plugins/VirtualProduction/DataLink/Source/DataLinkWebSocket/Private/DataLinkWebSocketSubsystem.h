// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "DataLinkWebSocketHandle.h"
#include "Subsystems/EngineSubsystem.h"
#include "DataLinkWebSocketSubsystem.generated.h"

class IWebSocket;
struct FDataLinkWebSocketSettings;

UCLASS()
class UDataLinkWebSocketSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	static UDataLinkWebSocketSubsystem* TryGet();

	struct FCreateWebSocketResult
	{
		UE::DataLink::FWebSocketHandle Handle;
		TSharedPtr<IWebSocket> WebSocket;
	};
	bool CreateWebSocket(const FDataLinkWebSocketSettings& InSettings, FCreateWebSocketResult& OutResult);

	void CloseWebSocket(UE::DataLink::FWebSocketHandle InHandle);

	/** Returns the web socket mapped to the handle if valid */
	TSharedPtr<IWebSocket> FindWebSocket(UE::DataLink::FWebSocketHandle InHandle) const;

private:
	void CleanupInvalidWebSockets();

	/** Registered web sockets */
	TMap<UE::DataLink::FWebSocketHandle, TSharedRef<IWebSocket>> WebSockets;

	/** Web sockets that have requested closure and are pending kill */
	TArray<TSharedRef<IWebSocket>> InvalidWebSockets;

	/** Handle to the callback to clean up invalid web sockets */
	FTSTicker::FDelegateHandle CleanupHandle;
};
