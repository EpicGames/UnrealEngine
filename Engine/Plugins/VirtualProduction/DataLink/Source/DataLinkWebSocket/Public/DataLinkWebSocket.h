// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkNode.h"
#include "DataLinkWebSocketHandle.h"
#include "DataLinkWebSocketSettings.h"
#include "DataLinkWebSocket.generated.h"

#define UE_API DATALINKWEBSOCKET_API

/** The messages to send through web socket */
USTRUCT(BlueprintType)
struct FDataLinkWebSocketMessages
{
	GENERATED_BODY()

	/** Messages to send */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Web Sockets")
	TArray<FString> ConnectMessages;
};

/** Instance data for the Web Socket Data Link node */
USTRUCT()
struct FDataLinkWebSocketInstanceData
{
	GENERATED_BODY()

	/** Handle to the last web socket created for this instance */
	UE::DataLink::FWebSocketHandle WebSocketHandle;

	/**
	 * Settings used to create the web socket
	 * Used to determine if a new web socket has to be created
	 */
	UPROPERTY()
	FDataLinkWebSocketSettings WebSocketSettings;
};

UCLASS(MinimalAPI, DisplayName="Web Socket", Category="Web Socket")
class UDataLinkWebSocket : public UDataLinkNode
{
	GENERATED_BODY()

	UDataLinkWebSocket();

protected:
	//~ Begin UDataLinkNode
	UE_API virtual void OnBuildPins(FDataLinkPinBuilder& Inputs, FDataLinkPinBuilder& Outputs) const override;
	UE_API virtual EDataLinkExecutionReply OnExecute(FDataLinkExecutor& InExecutor) const override;
	UE_API virtual void OnStop(const FDataLinkExecutor& InExecutor) const override;
	//~ End UDataLinkNode

private:
	/** Called when the web socket has connected */
	void OnConnected(TWeakPtr<FDataLinkExecutor> InExecutorWeak) const;

	/** Called when the web socket encountered a connection error */
	void OnConnectionError(const FString& InError, TWeakPtr<FDataLinkExecutor> InExecutorWeak) const;

	/** Called when the web socket has closed */
	void OnClosed(int32 InStatusCode, const FString& InReason, bool bInWasClean, TWeakPtr<FDataLinkExecutor> InExecutorWeak) const;

	/** Called when the web socket has a received a message */
	void OnMessageReceived(const FString& InMessage, TWeakPtr<FDataLinkExecutor> InExecutorWeak) const;
};

#undef UE_API
