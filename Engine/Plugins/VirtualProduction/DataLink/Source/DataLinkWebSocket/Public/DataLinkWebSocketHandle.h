// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/TypeHash.h"

namespace UE::DataLink
{

class FWebSocketHandle
{
public:
	static FWebSocketHandle GenerateNewHandle();

	void Reset()
	{
		Id = 0;
	}

	bool IsValid() const
	{
		return Id != 0;
	}

	bool operator==(const FWebSocketHandle& InOther) const
	{
		return Id == InOther.Id;
	}

	friend uint32 GetTypeHash(const FWebSocketHandle& InHandle)
	{
		return ::GetTypeHash(InHandle.Id);
	}

private:
	uint32 Id = 0;
};

} // UE::DataLink
