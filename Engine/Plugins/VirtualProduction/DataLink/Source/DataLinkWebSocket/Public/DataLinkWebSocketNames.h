// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::DataLink
{
	const FLazyName InputWebSocketSettings(TEXT("InputWebSocketSettings"));
	const FLazyName InputWebSocketMessages(TEXT("InputWebSocketMessages"));
}
