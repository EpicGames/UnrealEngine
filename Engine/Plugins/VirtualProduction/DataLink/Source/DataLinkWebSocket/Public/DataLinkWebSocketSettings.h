// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DataLinkWebSocketSettings.generated.h"

#define UE_API DATALINKWEBSOCKET_API

USTRUCT(BlueprintType)
struct FDataLinkWebSocketSettings
{
	GENERATED_BODY()

	/** Resets all the settings */
	UE_API void Reset();

	/** Returns true if the settings matches the given other settings */
	UE_API bool Equals(const FDataLinkWebSocketSettings& InOther) const;

	/** URL to connect to where the web socket server will respond */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Web Sockets")
	FString URL;

	/** List of the protocols to use with the connection */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Web Sockets")
	TArray<FString> Protocols;

	/** The upgrade headers to send with the upgrade request */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Data Link Web Sockets")
	TMap<FString, FString> UpgradeHeaders;
};

#undef UE_API
